// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveFunctionCollapse.h"
#include "Modules/ModuleManager.h"
#include "WaveFunctionCollapseEditorCommands.h"
#include "LevelEditor.h"
#include "EditorUtilitySubsystem.h"
#include "EditorUtilityWidgetBlueprint.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "FWaveFunctionCollapseModule"

void FWaveFunctionCollapseModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
	FWaveFunctionCollapseEditorCommands::Register();

	PluginCommands = MakeShareable(new FUICommandList);

	PluginCommands->MapAction(
		FWaveFunctionCollapseEditorCommands::Get().WaveFunctionCollapseWidget,
		FExecuteAction::CreateRaw(this, &FWaveFunctionCollapseModule::WaveFunctionCollapseUI)
	);

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");

	{
		TSharedPtr<FExtender> NewMenuExtender = MakeShareable(new FExtender);
		NewMenuExtender->AddMenuExtension("LevelEditor",
			EExtensionHook::After,
			PluginCommands,
			FMenuExtensionDelegate::CreateRaw(this, &FWaveFunctionCollapseModule::AddMenuEntry));

		LevelEditorModule.GetMenuExtensibilityManager()->AddExtender(NewMenuExtender);
	}
}

void FWaveFunctionCollapseModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
	FWaveFunctionCollapseEditorCommands::Unregister();
}

void FWaveFunctionCollapseModule::AddMenuEntry(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("CustomMenu", TAttribute<FText>(FText::FromString("WaveFunctionCollapse")));
	MenuBuilder.AddMenuEntry(FWaveFunctionCollapseEditorCommands::Get().WaveFunctionCollapseWidget);
	MenuBuilder.EndSection();
}

void FWaveFunctionCollapseModule::WaveFunctionCollapseUI()
{
	UEditorUtilitySubsystem* EUSubsystem = GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>();
	UEditorUtilityWidgetBlueprint* WaveFunctionCollapseEUWBP = LoadObject<UEditorUtilityWidgetBlueprint>(NULL, TEXT("/WaveFunctionCollapse/Core/WaveFunctionCollapse.WaveFunctionCollapse"), NULL, LOAD_None, NULL);
	UEditorUtilityWidget* WaveFunctionCollapseEUW = EUSubsystem->SpawnAndRegisterTab(WaveFunctionCollapseEUWBP);
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FWaveFunctionCollapseModule, WaveFunctionCollapse)
