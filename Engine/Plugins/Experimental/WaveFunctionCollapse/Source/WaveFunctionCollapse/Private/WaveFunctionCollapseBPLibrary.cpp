// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveFunctionCollapseBPLibrary.h"
#include "Engine/StaticMesh.h"
#include "WaveFunctionCollapseSubsystem.h"
#include "Engine/StaticMeshActor.h"
#include "Components/InstancedStaticMeshComponent.h"

UWaveFunctionCollapseBPLibrary::UWaveFunctionCollapseBPLibrary(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

float UWaveFunctionCollapseBPLibrary::CalculateShannonEntropy(const TArray<FWaveFunctionCollapseOption>& Options, UWaveFunctionCollapseModel* WFCModel)
{
	if (Options.IsEmpty())
	{
		UE_LOGF(LogWFC, Display, "Cannot calculate shannon entropy because the options are empty.");
		return -1;
	}

	float SumWeights = 0;
	float SumWeightXLogWeight = 0;
	for (const FWaveFunctionCollapseOption& Option : Options)
	{
		if (WFCModel->Constraints.Contains(Option))
		{
			const float& Weight = WFCModel->Constraints.FindRef(Option).Weight;
			SumWeights += Weight;
			SumWeightXLogWeight += (Weight * log(Weight));
		}
	}

	if (SumWeights == 0)
	{
		UE_LOGF(LogWFC, Display, "Cannot calculate shannon entropy because the sum of weights equals zero.");
		return -1;
	}

	return log(SumWeights) - (SumWeightXLogWeight / SumWeights);
}

int32 UWaveFunctionCollapseBPLibrary::PositionAsIndex(FIntVector Position, FIntVector Resolution)
{
	return Position.X + (Position.Y * Resolution.X) + (Position.Z * Resolution.X * Resolution.Y);
}

FIntVector UWaveFunctionCollapseBPLibrary::IndexAsPosition(int32 Index, FIntVector Resolution)
{
	FIntVector Position;
	Position.Z = floor(Index / (Resolution.X * Resolution.Y));
	Position.Y = floor((Index - (Position.Z * Resolution.X * Resolution.Y)) / Resolution.X);
	Position.X = Index - (Position.Y * Resolution.X) - (Position.Z * Resolution.X * Resolution.Y);
	return Position;
}

FWaveFunctionCollapseTile UWaveFunctionCollapseBPLibrary::BuildInitialTile(UWaveFunctionCollapseModel* WFCModel)
{
	FWaveFunctionCollapseTile InitialTile;
	for (TPair<FWaveFunctionCollapseOption, FWaveFunctionCollapseAdjacencyToOptionsMap>& Constraint : WFCModel->Constraints)
	{
		InitialTile.RemainingOptions.AddUnique(Constraint.Key);
	}
	InitialTile.ShannonEntropy = UWaveFunctionCollapseBPLibrary::CalculateShannonEntropy(InitialTile.RemainingOptions, WFCModel);
	return InitialTile;
}

TMap<int32, EWaveFunctionCollapseAdjacency> UWaveFunctionCollapseBPLibrary::GetAdjacentIndices(int32 Index, FIntVector Resolution)
{
	TMap<int32, EWaveFunctionCollapseAdjacency> AdjacentIndices;
	FIntVector Position = IndexAsPosition(Index, Resolution);
	if (Position.X + 1 < Resolution.X) 
	{ 
		AdjacentIndices.Add(PositionAsIndex(Position + FIntVector(1, 0, 0), Resolution),EWaveFunctionCollapseAdjacency::Front);
	}
	if (Position.X - 1 >= 0) 
	{ 
		AdjacentIndices.Add(PositionAsIndex(Position + FIntVector(-1, 0, 0), Resolution), EWaveFunctionCollapseAdjacency::Back); 
	}
	if (Position.Y + 1 < Resolution.Y) 
	{ 
		AdjacentIndices.Add(PositionAsIndex(Position + FIntVector(0, 1, 0), Resolution), EWaveFunctionCollapseAdjacency::Right); 
	}
	if (Position.Y - 1 >= 0) 
	{ 
		AdjacentIndices.Add(PositionAsIndex(Position + FIntVector(0, -1, 0), Resolution), EWaveFunctionCollapseAdjacency::Left); 
	}
	if (Position.Z + 1 < Resolution.Z) 
	{ 
		AdjacentIndices.Add(PositionAsIndex(Position + FIntVector(0, 0, 1), Resolution), EWaveFunctionCollapseAdjacency::Up); 
	}
	if (Position.Z - 1 >= 0) 
	{ 
		AdjacentIndices.Add(PositionAsIndex(Position + FIntVector(0, 0, -1), Resolution), EWaveFunctionCollapseAdjacency::Down); 
	}

	return AdjacentIndices;
}

TMap<FIntVector, EWaveFunctionCollapseAdjacency> UWaveFunctionCollapseBPLibrary::GetAdjacentPositions(FIntVector Position, FIntVector Resolution)
{
	TMap<FIntVector, EWaveFunctionCollapseAdjacency> AdjacentPositions;
	if (Position.X + 1 < Resolution.X) 
	{ 
		AdjacentPositions.Add(Position + FIntVector(1, 0, 0), EWaveFunctionCollapseAdjacency::Front); 
	}
	if (Position.X - 1 >= 0) 
	{ 
		AdjacentPositions.Add(Position + FIntVector(-1, 0, 0), EWaveFunctionCollapseAdjacency::Back); 
	}
	if (Position.Y + 1 < Resolution.Y) 
	{ 
		AdjacentPositions.Add(Position + FIntVector(0, 1, 0), EWaveFunctionCollapseAdjacency::Right); 
	}
	if (Position.Y - 1 >= 0) 
	{ 
		AdjacentPositions.Add(Position + FIntVector(0, -1, 0), EWaveFunctionCollapseAdjacency::Left); 
	}
	if (Position.Z + 1 < Resolution.Z) 
	{ 
		AdjacentPositions.Add(Position + FIntVector(0, 0, 1), EWaveFunctionCollapseAdjacency::Up); 
	}
	if (Position.Z - 1 >= 0) 
	{ 
		AdjacentPositions.Add(Position + FIntVector(0, 0, -1), EWaveFunctionCollapseAdjacency::Down); 
	}

	return AdjacentPositions;
}

bool UWaveFunctionCollapseBPLibrary::IsOptionContained(const FWaveFunctionCollapseOption& Option, const TArray<FWaveFunctionCollapseOption>& Options)
{
	for (const FWaveFunctionCollapseOption& CheckAgainst : Options)
	{
		if (Option == CheckAgainst)
		{
			return true;
		}
	}
	return false;
}

EWaveFunctionCollapseAdjacency UWaveFunctionCollapseBPLibrary::GetOppositeAdjacency(EWaveFunctionCollapseAdjacency Adjacency)
{
	switch (Adjacency)
	{
		case EWaveFunctionCollapseAdjacency::Front:
			return EWaveFunctionCollapseAdjacency::Back;
		case EWaveFunctionCollapseAdjacency::Back:
			return EWaveFunctionCollapseAdjacency::Front;
		case EWaveFunctionCollapseAdjacency::Right:
			return EWaveFunctionCollapseAdjacency::Left;
		case EWaveFunctionCollapseAdjacency::Left:
			return EWaveFunctionCollapseAdjacency::Right;
		case EWaveFunctionCollapseAdjacency::Up:
			return EWaveFunctionCollapseAdjacency::Down;
		case EWaveFunctionCollapseAdjacency::Down:
			return EWaveFunctionCollapseAdjacency::Up;
		default:
			return Adjacency;
	}
}

EWaveFunctionCollapseAdjacency UWaveFunctionCollapseBPLibrary::GetNextZAxisClockwiseAdjacency(EWaveFunctionCollapseAdjacency Adjacency)
{
	switch (Adjacency)
	{
	case EWaveFunctionCollapseAdjacency::Front:
		return EWaveFunctionCollapseAdjacency::Right;
	case EWaveFunctionCollapseAdjacency::Right:
		return EWaveFunctionCollapseAdjacency::Back;
	case EWaveFunctionCollapseAdjacency::Back:
		return EWaveFunctionCollapseAdjacency::Left;
	case EWaveFunctionCollapseAdjacency::Left:
		return EWaveFunctionCollapseAdjacency::Front;
	case EWaveFunctionCollapseAdjacency::Up:
		return EWaveFunctionCollapseAdjacency::Up;
	case EWaveFunctionCollapseAdjacency::Down:
		return EWaveFunctionCollapseAdjacency::Down;
	default:
		return Adjacency;
	}
}

void UWaveFunctionCollapseBPLibrary::AddToAdjacencyToOptionsMap(UPARAM(ref) FWaveFunctionCollapseAdjacencyToOptionsMap& AdjacencyToOptionsMap, EWaveFunctionCollapseAdjacency OptionAdjacency, FWaveFunctionCollapseOption OptionObject)
{
	if (FWaveFunctionCollapseOptions* Options = AdjacencyToOptionsMap.AdjacencyToOptionsMap.Find(OptionAdjacency))
	{
		// If the Options array does not contain the Option, create it
		if (!Options->Options.Contains(OptionObject))
		{
			Options->Options.Add(OptionObject);
		}
	}
	else // Create the Option and add it to the AdjacencyToOptionsMap
	{
		FWaveFunctionCollapseOptions NewOptions;
		NewOptions.Options.Add(OptionObject);
		AdjacencyToOptionsMap.AdjacencyToOptionsMap.Add(OptionAdjacency, NewOptions);
	}
}

bool UWaveFunctionCollapseBPLibrary::IsSoftObjPathEqual(const FSoftObjectPath& SoftObjectPathA, const FSoftObjectPath& SoftObjectPathB)
{
	return SoftObjectPathA == SoftObjectPathB;
}

FRotator UWaveFunctionCollapseBPLibrary::SanitizeRotator(FRotator Rotator)
{
	FRotator OutputRotator;
	OutputRotator = Rotator;

	// Round to orthogonal values
	OutputRotator.Roll = FMath::RoundHalfToEven(OutputRotator.Roll / 90) * 90;
	OutputRotator.Pitch = FMath::RoundHalfToEven(OutputRotator.Pitch / 90) * 90;
	OutputRotator.Yaw = FMath::RoundHalfToEven(OutputRotator.Yaw / 90) * 90;
	OutputRotator.Normalize();

	// Convert from Rotator to Quaternion to Matrix and back To Rotator to ensure single representation of rotation value
	FTransform MyTransform;
	MyTransform.SetRotation(FQuat(OutputRotator));
	FMatrix MyMatrix = MyTransform.ToMatrixNoScale();
	OutputRotator= MyMatrix.Rotator();

	// Ensure that -180 values are adjusted to 180
	OutputRotator.Normalize();
	OutputRotator.Roll = FMath::RoundHalfToEven(OutputRotator.Roll);
	if (OutputRotator.Roll == -180) 
	{ 
		OutputRotator.Roll = 180; 
	}
	OutputRotator.Pitch = FMath::RoundHalfToEven(OutputRotator.Pitch);
	if (OutputRotator.Pitch == -180) 
	{ 
		OutputRotator.Pitch = 180; 
	}
	OutputRotator.Yaw = FMath::RoundHalfToEven(OutputRotator.Yaw);
	if (OutputRotator.Yaw == -180) 
	{ 
		OutputRotator.Yaw = 180; 
	}

	return OutputRotator;
}

void UWaveFunctionCollapseBPLibrary::DeriveModelFromActors(UPARAM(ref) const TArray<AActor*>& InputActors, 
	UWaveFunctionCollapseModel* WFCModel, 
	float TileSize, 
	bool bIsBorderEmptyOption, 
	bool bIsMinZFloorOption,
	bool bUseUniformWeightDistribution,
	bool bAutoDeriveZAxisRotationConstraints,
	const TArray<FSoftObjectPath>& SpawnExclusionAssets,
	const TArray<FSoftObjectPath>& IgnoreRotationAssets)
{
	// Check if there are valid actors
	if (InputActors.IsEmpty())
	{
		return;
	}

	// Check if Model is valid
	if (!WFCModel)
	{
		return;
	}

	// Initialize Variables
	TMap<FIntVector, FWaveFunctionCollapseOption> PositionToOptionWithBorderMap;
	FIntVector ResolutionWithBorder;

	// Set TileSize
	WFCModel->Modify();
	WFCModel->TileSize = TileSize;

	// Create all Empty Option adjacency constraints
	WFCModel->AddConstraint(FWaveFunctionCollapseOption::EmptyOption, EWaveFunctionCollapseAdjacency::Front, FWaveFunctionCollapseOption::EmptyOption);
	WFCModel->AddConstraint(FWaveFunctionCollapseOption::EmptyOption, EWaveFunctionCollapseAdjacency::Back, FWaveFunctionCollapseOption::EmptyOption);
	WFCModel->AddConstraint(FWaveFunctionCollapseOption::EmptyOption, EWaveFunctionCollapseAdjacency::Left, FWaveFunctionCollapseOption::EmptyOption);
	WFCModel->AddConstraint(FWaveFunctionCollapseOption::EmptyOption, EWaveFunctionCollapseAdjacency::Right, FWaveFunctionCollapseOption::EmptyOption);
	WFCModel->AddConstraint(FWaveFunctionCollapseOption::EmptyOption, EWaveFunctionCollapseAdjacency::Up, FWaveFunctionCollapseOption::EmptyOption);
	WFCModel->AddConstraint(FWaveFunctionCollapseOption::EmptyOption, EWaveFunctionCollapseAdjacency::Down, FWaveFunctionCollapseOption::EmptyOption);
	
	// Derive Grid from Actors
	FIntVector MinPosition = FIntVector::ZeroValue;
	FIntVector MaxPosition = FIntVector::ZeroValue;
	FVector TmpOrigin = InputActors[0]->GetActorLocation(); // Set TmpOrigin based on first actor in the array
	TMap<FIntVector, FWaveFunctionCollapseOption> TmpPositionToOptionMap;
	for (int32 InputActorIndex = 0; InputActorIndex < InputActors.Num(); InputActorIndex++)
	{
		AStaticMeshActor* InputStaticMeshActor = Cast<AStaticMeshActor>(InputActors[InputActorIndex]);
		if (InputStaticMeshActor)
		{
			UStaticMesh* InputStaticMesh = InputStaticMeshActor->GetStaticMeshComponent()->GetStaticMesh();
			if (!InputStaticMesh)
			{
				UE_LOGF(LogWFC, Display, "%ls contains invalid StaticMesh, skipping.", *InputStaticMeshActor->GetFName().ToString());
				continue;
			}

			// Find CurrentGridPosition
			FVector CurrentGridPositionFloat = (InputStaticMeshActor->GetActorLocation() - TmpOrigin) / TileSize;
			FIntVector CurrentGridPosition = FIntVector(FMath::RoundHalfFromZero(CurrentGridPositionFloat.X), FMath::RoundHalfFromZero(CurrentGridPositionFloat.Y), FMath::RoundHalfFromZero(CurrentGridPositionFloat.Z));
			if (TmpPositionToOptionMap.Contains(CurrentGridPosition))
			{
				UE_LOGF(LogWFC, Display, "%ls is in an overlapping position, skipping.", *InputStaticMeshActor->GetFName().ToString());
				continue;
			}

			// Find min/max positions
			MinPosition.X = FMath::Min(MinPosition.X, CurrentGridPosition.X);
			MinPosition.Y = FMath::Min(MinPosition.Y, CurrentGridPosition.Y);
			MinPosition.Z = FMath::Min(MinPosition.Z, CurrentGridPosition.Z);
			MaxPosition.X = FMath::Max(MaxPosition.X, CurrentGridPosition.X);
			MaxPosition.Y = FMath::Max(MaxPosition.Y, CurrentGridPosition.Y);
			MaxPosition.Z = FMath::Max(MaxPosition.Z, CurrentGridPosition.Z);

			// Add to Temporary Position to Option Map
			FRotator CurrentRotator = FRotator::ZeroRotator;
			if (!IgnoreRotationAssets.Contains(FSoftObjectPath(InputStaticMesh->GetPathName())))
			{
				CurrentRotator = SanitizeRotator(InputStaticMeshActor->GetActorRotation());
			}
			FWaveFunctionCollapseOption CurrentOption(InputStaticMesh->GetPathName(), CurrentRotator, InputStaticMeshActor->GetActorScale3D());
			TmpPositionToOptionMap.Add(CurrentGridPosition, CurrentOption);
		}
	}

	// Re-order grid from 0,0,0 and add border
	for (TPair<FIntVector, FWaveFunctionCollapseOption>& TmpPositionToOption : TmpPositionToOptionMap)
	{
		PositionToOptionWithBorderMap.Add(TmpPositionToOption.Key - MinPosition + FIntVector(1), TmpPositionToOption.Value);
	}
	TmpPositionToOptionMap.Empty();

	// Set Resolution with Border
	ResolutionWithBorder = MaxPosition - MinPosition + FIntVector(3);
	
	// Derive Constraints
	for (TPair<FIntVector, FWaveFunctionCollapseOption>& PositionToOptionWithBorder : PositionToOptionWithBorderMap)
	{
		TMap<FIntVector, EWaveFunctionCollapseAdjacency> PositionToAdjacenciesMap = GetAdjacentPositions(PositionToOptionWithBorder.Key, ResolutionWithBorder);
		for (TPair<FIntVector, EWaveFunctionCollapseAdjacency>& PositionToAdjacencies : PositionToAdjacenciesMap)
		{
			FIntVector PositionToCheck = PositionToAdjacencies.Key;

			// if Adjacent Option exists, add constraint to model
			if (PositionToOptionWithBorderMap.Contains(PositionToCheck))
			{
				WFCModel->AddConstraint(PositionToOptionWithBorder.Value, PositionToAdjacencies.Value, PositionToOptionWithBorderMap.FindRef(PositionToCheck));
			}
			// if MinZ and bIsMinZFloorOption, add Border Option constraint
			else if (bIsMinZFloorOption && PositionToCheck.Z == 0)
			{
				WFCModel->AddConstraint(FWaveFunctionCollapseOption::BorderOption, GetOppositeAdjacency(PositionToAdjacencies.Value), PositionToOptionWithBorder.Value);
			}
			// if ExteriorBorder and bIsBorderEmptyOption, add Empty Option and Inverse constraints
			else if (PositionToCheck.X == 0
				|| PositionToCheck.Y == 0
				|| PositionToCheck.Z == 0
				|| PositionToCheck.X == ResolutionWithBorder.X - 1
				|| PositionToCheck.Y == ResolutionWithBorder.Y - 1
				|| PositionToCheck.Z == ResolutionWithBorder.Z - 1)
			{
				if (bIsBorderEmptyOption)
				{
					WFCModel->AddConstraint(PositionToOptionWithBorder.Value, PositionToAdjacencies.Value, FWaveFunctionCollapseOption::EmptyOption);
					WFCModel->AddConstraint(FWaveFunctionCollapseOption::EmptyOption, GetOppositeAdjacency(PositionToAdjacencies.Value), PositionToOptionWithBorder.Value);
				}
			}
			// otherwise it is an empty space, add Empty Option and Inverse constraints
			else
			{
				WFCModel->AddConstraint(PositionToOptionWithBorder.Value, PositionToAdjacencies.Value, FWaveFunctionCollapseOption::EmptyOption);
				WFCModel->AddConstraint(FWaveFunctionCollapseOption::EmptyOption, GetOppositeAdjacency(PositionToAdjacencies.Value), PositionToOptionWithBorder.Value);
			}
		}
	}
	PositionToOptionWithBorderMap.Empty();

	// Auto Derive ZAxis Rotation Constraints
	if (bAutoDeriveZAxisRotationConstraints)
	{
		TArray<FWaveFunctionCollapseOption> NewKeyOptions;
		TArray<EWaveFunctionCollapseAdjacency> NewAdjacencies;
		TArray<FWaveFunctionCollapseOption> NewAdjacentOptions;

		for (const TPair<FWaveFunctionCollapseOption, FWaveFunctionCollapseAdjacencyToOptionsMap>& Constraint : WFCModel->Constraints)
		{
			for (const TPair<EWaveFunctionCollapseAdjacency, FWaveFunctionCollapseOptions>& AdjacencyToOptionsPair : Constraint.Value.AdjacencyToOptionsMap)
			{
				for (const FWaveFunctionCollapseOption& AdjacentOption : AdjacencyToOptionsPair.Value.Options)
				{
					FWaveFunctionCollapseOption NewKeyOption = Constraint.Key;
					FWaveFunctionCollapseOption NewAdjacentOption = AdjacentOption;
					EWaveFunctionCollapseAdjacency NewAdjacency = AdjacencyToOptionsPair.Key;

					int32 KeyOptionRotationMultiplier = 0;
					if (!(IgnoreRotationAssets.Contains(NewKeyOption.BaseObject)
						|| NewKeyOption == FWaveFunctionCollapseOption::EmptyOption
						|| NewKeyOption == FWaveFunctionCollapseOption::BorderOption
						|| NewKeyOption == FWaveFunctionCollapseOption::VoidOption))
					{
						KeyOptionRotationMultiplier = 1;
					}

					int32 AdjacentOptionRotationMultiplier = 0;
					if (!(IgnoreRotationAssets.Contains(NewAdjacentOption.BaseObject)
						|| NewAdjacentOption == FWaveFunctionCollapseOption::EmptyOption
						|| NewAdjacentOption == FWaveFunctionCollapseOption::BorderOption
						|| NewAdjacentOption == FWaveFunctionCollapseOption::VoidOption))
					{
						AdjacentOptionRotationMultiplier = 1;
					}

					for (int32 RotationIncrement = 1; RotationIncrement <= 3; RotationIncrement++)
					{
						NewKeyOption.BaseRotator.Yaw += (90.0f * KeyOptionRotationMultiplier);
						NewKeyOption.BaseRotator = SanitizeRotator(NewKeyOption.BaseRotator);
						NewAdjacentOption.BaseRotator.Yaw += (90.0f * AdjacentOptionRotationMultiplier);
						NewAdjacentOption.BaseRotator = SanitizeRotator(NewAdjacentOption.BaseRotator);
						NewAdjacency = GetNextZAxisClockwiseAdjacency(NewAdjacency);

						// Store new constraints
						NewKeyOptions.Add(NewKeyOption);
						NewAdjacencies.Add(NewAdjacency);
						NewAdjacentOptions.Add(NewAdjacentOption);
					}
				}
			}
		}

		// Add stored new constraints to the model
		for (int32 Index = 0; Index < NewKeyOptions.Num(); Index++)
		{
			WFCModel->AddConstraint(NewKeyOptions[Index], NewAdjacencies[Index], NewAdjacentOptions[Index]);
		}
	}
	
	// If Empty->OptionA constraint and Empty<-OptionB exist, store OptionA->OptionB constraint
	TArray<FWaveFunctionCollapseOption> EmptyVoidAdjacentKeyOptions;
	TArray<EWaveFunctionCollapseAdjacency> EmptyVoidAdjacentAdjacencies;
	TArray<FWaveFunctionCollapseOption> EmptyVoidAdjacentAdjacentOptions;
	const FWaveFunctionCollapseAdjacencyToOptionsMap& EmptyOptionConstraints = WFCModel->Constraints.FindRef(FWaveFunctionCollapseOption::EmptyOption);
	for (const TPair<EWaveFunctionCollapseAdjacency, FWaveFunctionCollapseOptions>& AdjacencyToOptionsPair : EmptyOptionConstraints.AdjacencyToOptionsMap)
	{
		for (const FWaveFunctionCollapseOption& EmptyAdjacentOption : AdjacencyToOptionsPair.Value.Options)
		{
			if (EmptyAdjacentOption == FWaveFunctionCollapseOption::EmptyOption)
			{
				continue;
			}

			if (const FWaveFunctionCollapseOptions* OppositeEmptyAdjacenctOptions = EmptyOptionConstraints.AdjacencyToOptionsMap.Find(GetOppositeAdjacency(AdjacencyToOptionsPair.Key)))
			{
				for (const FWaveFunctionCollapseOption& OppositeEmptyAdjacentOption : OppositeEmptyAdjacenctOptions->Options)
				{
					if (OppositeEmptyAdjacentOption == FWaveFunctionCollapseOption::EmptyOption)
					{
						continue;
					}

					EmptyVoidAdjacentKeyOptions.Add(EmptyAdjacentOption);
					EmptyVoidAdjacentAdjacencies.Add(GetOppositeAdjacency(AdjacencyToOptionsPair.Key));
					EmptyVoidAdjacentAdjacentOptions.Add(OppositeEmptyAdjacentOption);
				}
			}
		}
	}

	// If Void->OptionA constraint and Void<-OptionB exist, store OptionA->OptionB constraint
	const FWaveFunctionCollapseAdjacencyToOptionsMap& VoidOptionConstraints = WFCModel->Constraints.FindRef(FWaveFunctionCollapseOption::VoidOption);
	for (const TPair<EWaveFunctionCollapseAdjacency, FWaveFunctionCollapseOptions>& AdjacencyToOptionsPair : VoidOptionConstraints.AdjacencyToOptionsMap)
	{
		for (const FWaveFunctionCollapseOption& VoidAdjacentOption : AdjacencyToOptionsPair.Value.Options)
		{
			if (VoidAdjacentOption == FWaveFunctionCollapseOption::VoidOption)
			{
				continue;
			}

			if (const FWaveFunctionCollapseOptions* OppositeVoidAdjacenctOptions = VoidOptionConstraints.AdjacencyToOptionsMap.Find(GetOppositeAdjacency(AdjacencyToOptionsPair.Key)))
			{
				for (const FWaveFunctionCollapseOption& OppositeVoidAdjacentOption : OppositeVoidAdjacenctOptions->Options)
				{
					if (VoidAdjacentOption == FWaveFunctionCollapseOption::VoidOption)
					{
						continue;
					}

					EmptyVoidAdjacentKeyOptions.Add(VoidAdjacentOption);
					EmptyVoidAdjacentAdjacencies.Add(GetOppositeAdjacency(AdjacencyToOptionsPair.Key));
					EmptyVoidAdjacentAdjacentOptions.Add(OppositeVoidAdjacentOption);
				}
			}
		}
	}

	// Add stored new constraints to the model
	for (int32 Index = 0; Index < EmptyVoidAdjacentKeyOptions.Num(); Index++)
	{
		WFCModel->AddConstraint(EmptyVoidAdjacentKeyOptions[Index], EmptyVoidAdjacentAdjacencies[Index], EmptyVoidAdjacentAdjacentOptions[Index]);
	}
	
	// Set Contributions
	if (bUseUniformWeightDistribution)
	{
		WFCModel->SetAllContributions(1);
	}
	WFCModel->SetOptionContribution(FWaveFunctionCollapseOption::BorderOption, 0);
	WFCModel->SetWeightsFromContributions();

	// Append to SpawnExlusion
	for (FSoftObjectPath SpawnExclusionAsset : SpawnExclusionAssets)
	{
		if (SpawnExclusionAsset.IsValid())
		{
			WFCModel->SpawnExclusion.AddUnique(SpawnExclusionAsset);
		}
	}
}

bool UWaveFunctionCollapseBPLibrary::GetPositionToOptionMapFromActor(AActor* Actor, float TileSize, UPARAM(ref) TMap<FIntVector, FWaveFunctionCollapseOption>& PositionToOptionMap)
{
	// Check if Model is valid
	if (!Actor)
	{
		UE_LOGF(LogWFC, Display, "GetPositionToOptionMapFromActor called with a null Actor");
		return false;
	}

	// Derive Grid from Actors
	FIntVector MinPosition = FIntVector::ZeroValue;
	FIntVector MaxPosition = FIntVector::ZeroValue;
	TMap<FIntVector, FWaveFunctionCollapseOption> TmpPositionToOptionMap;
	
	// Gather ISMComponents
	TInlineComponentArray<UInstancedStaticMeshComponent*> ISMComponents;
	Actor->GetComponents(ISMComponents);
	for (int32 ISMComponentIndex = 0; ISMComponentIndex < ISMComponents.Num(); ISMComponentIndex++)
	{
		UInstancedStaticMeshComponent* ISMComponent = ISMComponents[ISMComponentIndex];
		UStaticMesh* InputStaticMesh = ISMComponent->GetStaticMesh();
		if (!InputStaticMesh)
		{
			UE_LOGF(LogWFC, Display, "%ls.%ls contains invalid StaticMesh, skipping.", *Actor->GetFName().ToString(), *ISMComponent->GetFName().ToString());
			continue;
		}
		else
		{
			for (int32 InstanceIndex = 0; InstanceIndex < ISMComponent->GetInstanceCount(); InstanceIndex++)
			{
				FTransform InstanceTransform;
				ISMComponent->GetInstanceTransform(InstanceIndex, InstanceTransform);

				// Find CurrentGridPosition
				FVector CurrentGridPositionFloat = InstanceTransform.GetLocation() / TileSize;
				FIntVector CurrentGridPosition = FIntVector(FMath::RoundHalfFromZero(CurrentGridPositionFloat.X), FMath::RoundHalfFromZero(CurrentGridPositionFloat.Y), FMath::RoundHalfFromZero(CurrentGridPositionFloat.Z));
				if (TmpPositionToOptionMap.Contains(CurrentGridPosition))
				{
					UE_LOGF(LogWFC, Display, "%ls.%ls Index %d is in an overlapping position, skipping.", *Actor->GetFName().ToString(), *ISMComponent->GetFName().ToString(), InstanceIndex);
					continue;
				}

				// Find min/max positions
				MinPosition.X = FMath::Min(MinPosition.X, CurrentGridPosition.X);
				MinPosition.Y = FMath::Min(MinPosition.Y, CurrentGridPosition.Y);
				MinPosition.Z = FMath::Min(MinPosition.Z, CurrentGridPosition.Z);
				MaxPosition.X = FMath::Max(MaxPosition.X, CurrentGridPosition.X);
				MaxPosition.Y = FMath::Max(MaxPosition.Y, CurrentGridPosition.Y);
				MaxPosition.Z = FMath::Max(MaxPosition.Z, CurrentGridPosition.Z);

				// Add to Temporary Position to Option Map
				FWaveFunctionCollapseOption CurrentOption(InputStaticMesh->GetPathName(), InstanceTransform.Rotator(), InstanceTransform.GetScale3D());
				TmpPositionToOptionMap.Add(CurrentGridPosition, CurrentOption);
			}
		}
	}

	// Re-order grid 
	for (TPair<FIntVector, FWaveFunctionCollapseOption>& TmpPositionToOption : TmpPositionToOptionMap)
	{
		PositionToOptionMap.Add(TmpPositionToOption.Key - MinPosition, TmpPositionToOption.Value);
	}
	
	return !PositionToOptionMap.IsEmpty();
}

FWaveFunctionCollapseOption UWaveFunctionCollapseBPLibrary::MakeEmptyOption()
{
	return FWaveFunctionCollapseOption::EmptyOption;
}

FWaveFunctionCollapseOption UWaveFunctionCollapseBPLibrary::MakeBorderOption()
{
	return FWaveFunctionCollapseOption::BorderOption;
}

FWaveFunctionCollapseOption UWaveFunctionCollapseBPLibrary::MakeVoidOption()
{
	return FWaveFunctionCollapseOption::VoidOption;
}
