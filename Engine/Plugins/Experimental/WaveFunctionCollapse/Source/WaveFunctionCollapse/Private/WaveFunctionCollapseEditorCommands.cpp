// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveFunctionCollapseEditorCommands.h"

#define LOCTEXT_NAMESPACE "FWaveFunctionCollapseModule"

void FWaveFunctionCollapseEditorCommands::RegisterCommands()
{
	UI_COMMAND(WaveFunctionCollapseWidget, "WaveFunctionCollapse", "Wave Function Collapse Tool", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
