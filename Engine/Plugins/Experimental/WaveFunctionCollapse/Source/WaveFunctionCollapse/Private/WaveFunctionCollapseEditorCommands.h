// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/AppStyle.h"
#include "Framework/Commands/Commands.h"

class FWaveFunctionCollapseEditorCommands : public TCommands<FWaveFunctionCollapseEditorCommands>
{
public:

	FWaveFunctionCollapseEditorCommands()
		: TCommands<FWaveFunctionCollapseEditorCommands>
		(
			TEXT("WaveFunctionCollapse"),
			NSLOCTEXT("Contexts", "WaveFunctionCollapse", "WaveFunctionCollapse Plugin"),
			NAME_None,
			FAppStyle::GetAppStyleSetName()
		) 
		{}

	virtual void RegisterCommands() override;

public:

	TSharedPtr<FUICommandInfo> WaveFunctionCollapseWidget;
};