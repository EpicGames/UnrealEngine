// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveFunctionCollapseModel.h"
#include "Engine/StaticMesh.h"

const FWaveFunctionCollapseOption FWaveFunctionCollapseOption::EmptyOption(FString(TEXT("/WaveFunctionCollapse/Core/SpecialOptions/Option_Empty.Option_Empty")));
const FWaveFunctionCollapseOption FWaveFunctionCollapseOption::BorderOption(FString(TEXT("/WaveFunctionCollapse/Core/SpecialOptions/Option_Border.Option_Border")));
const FWaveFunctionCollapseOption FWaveFunctionCollapseOption::VoidOption(FString(TEXT("/WaveFunctionCollapse/Core/SpecialOptions/Option_Void.Option_Void")));

void UWaveFunctionCollapseModel::AddConstraint(const FWaveFunctionCollapseOption& KeyOption, EWaveFunctionCollapseAdjacency Adjacency, const FWaveFunctionCollapseOption& AdjacentOption)
{
	Modify(true);
	// If the KeyOption key exists in Constraints
	if (FWaveFunctionCollapseAdjacencyToOptionsMap* AdjacencyToOptionsMap = Constraints.Find(KeyOption))
	{
		// If the Adjacency key exists in the AdjacencyToOptionsMap
		if (FWaveFunctionCollapseOptions* Options = AdjacencyToOptionsMap->AdjacencyToOptionsMap.Find(Adjacency))
		{
			// If the Options array does not contain the Option, create it
			if (!Options->Options.Contains(AdjacentOption))
			{
				Options->Options.Add(AdjacentOption);
				AdjacencyToOptionsMap->Contribution += 1;
			}
		}
		else // Create the Option and add it to the AdjacencyToOptionsMap
		{
			FWaveFunctionCollapseOptions NewOptions;
			NewOptions.Options.Add(AdjacentOption);
			AdjacencyToOptionsMap->AdjacencyToOptionsMap.Add(Adjacency, NewOptions);
			AdjacencyToOptionsMap->Contribution += 1;
		}
	}
	else // Create the Option, add it to the AdjacencyToOptionsMap, and add it the Constraints map
	{
		FWaveFunctionCollapseOptions NewOptions;
		NewOptions.Options.Add(AdjacentOption);
		FWaveFunctionCollapseAdjacencyToOptionsMap NewAdjacencyToOptionsMap;
		NewAdjacencyToOptionsMap.AdjacencyToOptionsMap.Add(Adjacency, NewOptions);
		Constraints.Add(KeyOption, NewAdjacencyToOptionsMap);
	}
	
}

FWaveFunctionCollapseOptions UWaveFunctionCollapseModel::GetOptions(const FWaveFunctionCollapseOption& KeyOption, EWaveFunctionCollapseAdjacency Adjacency) const
{
	// If KeyOption key exists
	if (const FWaveFunctionCollapseAdjacencyToOptionsMap* AdjacencyToOptionsMap = Constraints.Find(KeyOption))
	{
		// If Adjacency key exists, return FoundOptions
		if (const FWaveFunctionCollapseOptions* FoundOptions = AdjacencyToOptionsMap->AdjacencyToOptionsMap.Find(Adjacency))
		{
			return *FoundOptions;
		}
	}
	// If nothing is found above, return empty
	return FWaveFunctionCollapseOptions();
}

void UWaveFunctionCollapseModel::SetWeightsFromContributions()
{
	if (Constraints.IsEmpty())
	{
		return;
	}

	Modify(true);
	int32 SumOfContributions = 0;
	for (TPair<FWaveFunctionCollapseOption, FWaveFunctionCollapseAdjacencyToOptionsMap>& Constraint : Constraints)
	{
		SumOfContributions += Constraint.Value.Contribution;
	}

	for (TPair<FWaveFunctionCollapseOption, FWaveFunctionCollapseAdjacencyToOptionsMap>& Constraint : Constraints)
	{
		Constraint.Value.Weight = float(Constraint.Value.Contribution) / SumOfContributions;
	}
}

void UWaveFunctionCollapseModel::SetAllWeights(float Weight)
{
	if (Constraints.IsEmpty())
	{
		return;
	}

	Modify(true);
	for (TPair<FWaveFunctionCollapseOption, FWaveFunctionCollapseAdjacencyToOptionsMap>& Constraint : Constraints)
	{
		Constraint.Value.Weight = Weight;
	}
}

void UWaveFunctionCollapseModel::SetAllContributions(int32 Contribution)
{
	if (Constraints.IsEmpty())
	{
		return;
	}

	Modify(true);
	for (TPair<FWaveFunctionCollapseOption, FWaveFunctionCollapseAdjacencyToOptionsMap>& Constraint : Constraints)
	{
		Constraint.Value.Contribution = Contribution;
	}
}

void UWaveFunctionCollapseModel::SetOptionContribution(const FWaveFunctionCollapseOption& Option, int32 Contribution)
{
	if (Constraints.Contains(Option))
	{
		Modify(true);
		FWaveFunctionCollapseAdjacencyToOptionsMap& AdjacencyToOptionsMap = *Constraints.Find(Option);
		AdjacencyToOptionsMap.Contribution = Contribution;
	}
}

float UWaveFunctionCollapseModel::GetOptionWeight(const FWaveFunctionCollapseOption& Option) const
{
	if (const FWaveFunctionCollapseAdjacencyToOptionsMap* Constraint = Constraints.Find(Option))
	{
		return Constraint->Weight;
	}
	else
	{
		return 0;
	}
}

int32 UWaveFunctionCollapseModel::GetOptionContribution(const FWaveFunctionCollapseOption& Option) const
{
	if (const FWaveFunctionCollapseAdjacencyToOptionsMap* Constraint = Constraints.Find(Option))
	{
		return Constraint->Contribution;
	}
	else
	{
		return 0;
	}
}

int32 UWaveFunctionCollapseModel::GetConstraintCount() const
{
	int32 ConstraintCount = 0;
	for (const TPair<FWaveFunctionCollapseOption, FWaveFunctionCollapseAdjacencyToOptionsMap>& Constraint : Constraints)
	{
		for (const TPair<EWaveFunctionCollapseAdjacency, FWaveFunctionCollapseOptions>& AdjacencyToOptions : Constraint.Value.AdjacencyToOptionsMap)
		{
			ConstraintCount += AdjacencyToOptions.Value.Options.Num();
		}
	}
	return ConstraintCount;
}

void UWaveFunctionCollapseModel::SwapMeshes(TMap<UStaticMesh*, UStaticMesh*> SourceToTargetMeshMap)
{
	Modify(true);
	for (TPair<UStaticMesh*, UStaticMesh*>& SourceToTargetMesh : SourceToTargetMeshMap)
	{
		UStaticMesh* SourceMesh = SourceToTargetMesh.Key;
		UStaticMesh* TargetMesh = SourceToTargetMesh.Value;
		for (TPair<FWaveFunctionCollapseOption, FWaveFunctionCollapseAdjacencyToOptionsMap>& Constraint : Constraints)
		{
			if (Constraint.Key.BaseObject == SourceMesh)
			{
				Constraint.Key.BaseObject = TargetMesh;
			}

			for (TPair<EWaveFunctionCollapseAdjacency, FWaveFunctionCollapseOptions>& AdjacencyToOptions : Constraint.Value.AdjacencyToOptionsMap)
			{
				for (FWaveFunctionCollapseOption& Option : AdjacencyToOptions.Value.Options)
				{
					if (Option.BaseObject == SourceMesh)
					{
						Option.BaseObject = TargetMesh;
					}
				}
			}
		}
	}
}
