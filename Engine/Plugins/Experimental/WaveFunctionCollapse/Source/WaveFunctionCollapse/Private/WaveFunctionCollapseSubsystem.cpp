// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveFunctionCollapseSubsystem.h"
#include "Engine/Blueprint.h"
#include "Engine/StaticMesh.h"
#include "WaveFunctionCollapseBPLibrary.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Subsystems/EditorActorSubsystem.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Editor.h"

DEFINE_LOG_CATEGORY(LogWFC);

AActor* UWaveFunctionCollapseSubsystem::Collapse(int32 TryCount /* = 1 */, int32 RandomSeed /* = 0 */)
{
	if (!WFCModel)
	{
		UE_LOGF(LogWFC, Error, "Invalid WFC Model");
		return nullptr;
	}
	
	UE_LOGF(LogWFC, Display, "Starting WFC - Model: %ls, Resolution %dx%dx%d", *WFCModel->GetFName().ToString(), Resolution.X, Resolution.Y, Resolution.Z);

	// Determinism settings
	int32 ChosenRandomSeed = (RandomSeed != 0 ? RandomSeed : FMath::RandRange(1, TNumericLimits<int32>::Max()));

	int32 ArrayReserveValue = Resolution.X * Resolution.Y * Resolution.Z;
	TArray<FWaveFunctionCollapseTile> Tiles;
	TArray<int32> RemainingTiles;
	TMap<int32, FWaveFunctionCollapseQueueElement> ObservationQueue;
	Tiles.Reserve(ArrayReserveValue);
	RemainingTiles.Reserve(ArrayReserveValue);

	InitializeWFC(Tiles, RemainingTiles);
	
	bool bSuccessfulSolve = false;

	if (TryCount > 1)
	{
		//Copy Original Initialized tiles
		TArray<FWaveFunctionCollapseTile> TilesCopy = Tiles;
		TArray<int32> RemainingTilesCopy = RemainingTiles;

		int32 CurrentTry = 1;
		bSuccessfulSolve = ObservationPropagation(Tiles, RemainingTiles, ObservationQueue, ChosenRandomSeed);
		FRandomStream RandomStream(ChosenRandomSeed);
		while (!bSuccessfulSolve && CurrentTry<TryCount)
		{
			CurrentTry += 1;
			UE_LOGF(LogWFC, Warning, "Failed with Seed Value: %d. Trying again.  Attempt number: %d", ChosenRandomSeed, CurrentTry);
			ChosenRandomSeed = RandomStream.RandRange(1, TNumericLimits<int32>::Max());
			
			// Start from Original Initialized tiles
			Tiles = TilesCopy;
			RemainingTiles = RemainingTilesCopy;
			ObservationQueue.Empty();
			bSuccessfulSolve = ObservationPropagation(Tiles, RemainingTiles, ObservationQueue, ChosenRandomSeed);
		}
	}
	else if (TryCount == 1)
	{
		bSuccessfulSolve = ObservationPropagation(Tiles, RemainingTiles, ObservationQueue, ChosenRandomSeed);
	}
	else
	{
		UE_LOGF(LogWFC, Error, "Invalid TryCount on Collapse: %d", TryCount);
		return nullptr;
	}

	// if Successful, Spawn Actor
	if (bSuccessfulSolve)
	{
		AActor* SpawnedActor = SpawnActorFromTiles(Tiles);
		UE_LOGF(LogWFC, Display, "Success! Seed Value: %d. Spawned Actor: %ls", ChosenRandomSeed, *SpawnedActor->GetActorLabel());
		return SpawnedActor;
	}
	else
	{
		UE_LOGF(LogWFC, Error, "Failed after %d tries.", TryCount);
		return nullptr;
	}
}

void UWaveFunctionCollapseSubsystem::InitializeWFC(TArray<FWaveFunctionCollapseTile>& Tiles, TArray<int32>& RemainingTiles)
{
	FWaveFunctionCollapseTile InitialTile;
	int32 SwapIndex = 0;

	if (BuildInitialTile(InitialTile))
	{
		float MinEntropy = InitialTile.ShannonEntropy;
		for (int32 Z = 0;Z < Resolution.Z; Z++)
		{
			for (int32 Y = 0;Y < Resolution.Y; Y++)
			{
				for (int32 X = 0;X < Resolution.X; X++)
				{
					// Pre-populate with starter tiles
					if (FWaveFunctionCollapseOption* StarterOption = StarterOptions.Find(FIntVector(X, Y, Z)))
					{
						FWaveFunctionCollapseTile StarterTile;
						StarterTile.RemainingOptions.Add(*StarterOption);
						StarterTile.ShannonEntropy = UWaveFunctionCollapseBPLibrary::CalculateShannonEntropy(StarterTile.RemainingOptions, WFCModel);
						Tiles.Add(StarterTile);
						RemainingTiles.Add(UWaveFunctionCollapseBPLibrary::PositionAsIndex(FIntVector(X, Y, Z), Resolution));
						
						// swap lower entropy tile to the beginning of RemainingTiles
						if (StarterTile.ShannonEntropy < MinEntropy)
						{
							RemainingTiles.Swap(0, RemainingTiles.Num() - 1);
							MinEntropy = StarterTile.ShannonEntropy;
							SwapIndex = 0;
						}
						
						// else, swap min entropy tile with the previous min entropy index+1
						else if (StarterTile.ShannonEntropy == MinEntropy && StarterTile.ShannonEntropy != InitialTile.ShannonEntropy)
						{
							SwapIndex += 1;
							RemainingTiles.Swap(SwapIndex, RemainingTiles.Num() - 1);
						}
					}
					
					// Pre-populate with border tiles
					else if (IsPositionInnerBorder(FIntVector(X, Y, Z))
						&& (bUseEmptyBorder || WFCModel->Constraints.Contains(FWaveFunctionCollapseOption::BorderOption)))
					{
						FWaveFunctionCollapseTile BorderTile;
						BorderTile.RemainingOptions = GetInnerBorderOptions(FIntVector(X, Y, Z), InitialTile.RemainingOptions);
						BorderTile.ShannonEntropy = UWaveFunctionCollapseBPLibrary::CalculateShannonEntropy(BorderTile.RemainingOptions, WFCModel);
						Tiles.Add(BorderTile);
						RemainingTiles.Add(UWaveFunctionCollapseBPLibrary::PositionAsIndex(FIntVector(X, Y, Z), Resolution));

						// swap lower entropy tile to the beginning of RemainingTiles
						if (BorderTile.ShannonEntropy < MinEntropy)
						{
							RemainingTiles.Swap(0, RemainingTiles.Num() - 1);
							MinEntropy = BorderTile.ShannonEntropy;
							SwapIndex = 0;
						}
						
						// else, swap min entropy tile with the previous min entropy index+1
						else if (BorderTile.ShannonEntropy ==  MinEntropy && BorderTile.ShannonEntropy != InitialTile.ShannonEntropy)
						{
							SwapIndex += 1;
							RemainingTiles.Swap(SwapIndex, RemainingTiles.Num() - 1);
						}
					}
					
					// Fill the rest with initial tiles
					else
					{
						Tiles.Add(InitialTile);
						RemainingTiles.Add(UWaveFunctionCollapseBPLibrary::PositionAsIndex(FIntVector(X, Y, Z), Resolution));
					}
				}
			}
		}
		StarterOptions.Empty();
	}
	else
	{
		UE_LOGF(LogWFC, Error, "Could not create Initial Tile from Model");
	}

}

bool UWaveFunctionCollapseSubsystem::BuildInitialTile(FWaveFunctionCollapseTile& InitialTile)
{
	TArray<FWaveFunctionCollapseOption> InitialOptions;
	for (const TPair< FWaveFunctionCollapseOption, FWaveFunctionCollapseAdjacencyToOptionsMap>& Constraint : WFCModel->Constraints)
	{
		if (Constraint.Key.BaseObject != FWaveFunctionCollapseOption::BorderOption.BaseObject)
		{
			InitialOptions.Add(Constraint.Key);
		}
	}

	if (!InitialOptions.IsEmpty())
	{
		InitialTile.RemainingOptions = InitialOptions;
		InitialTile.ShannonEntropy = UWaveFunctionCollapseBPLibrary::CalculateShannonEntropy(InitialOptions, WFCModel);
		return true;
	}
	else
	{
		return false;
	}
}

TArray<FWaveFunctionCollapseOption> UWaveFunctionCollapseSubsystem::GetInnerBorderOptions(FIntVector Position, const TArray<FWaveFunctionCollapseOption>& InitialOptions)
{
	TArray<FWaveFunctionCollapseOption> InnerBorderOptions;
	TArray<FWaveFunctionCollapseOption> InnerBorderOptionsToRemove;
	InnerBorderOptions = InitialOptions;
	
	// gather options to remove
	if (Position.X == 0)
	{
		GatherInnerBorderOptionsToRemove(EWaveFunctionCollapseAdjacency::Front, InnerBorderOptions, InnerBorderOptionsToRemove);
	}
	if (Position.X == Resolution.X - 1)
	{
		GatherInnerBorderOptionsToRemove(EWaveFunctionCollapseAdjacency::Back, InnerBorderOptions, InnerBorderOptionsToRemove);
	}
	if (Position.Y == 0)
	{
		GatherInnerBorderOptionsToRemove(EWaveFunctionCollapseAdjacency::Right, InnerBorderOptions, InnerBorderOptionsToRemove);
	}
	if (Position.Y == Resolution.Y - 1)
	{
		GatherInnerBorderOptionsToRemove(EWaveFunctionCollapseAdjacency::Left, InnerBorderOptions, InnerBorderOptionsToRemove);
	}
	if (Position.Z == 0)
	{
		GatherInnerBorderOptionsToRemove(EWaveFunctionCollapseAdjacency::Up, InnerBorderOptions, InnerBorderOptionsToRemove);
	}
	if (Position.Z == Resolution.Z - 1)
	{
		GatherInnerBorderOptionsToRemove(EWaveFunctionCollapseAdjacency::Down, InnerBorderOptions, InnerBorderOptionsToRemove);
	}
	
	//remove options
	if (!InnerBorderOptionsToRemove.IsEmpty())
	{
		for (FWaveFunctionCollapseOption& RemoveThisOption : InnerBorderOptionsToRemove)
		{
			InnerBorderOptions.RemoveSingleSwap(RemoveThisOption, EAllowShrinking::No);
		}
		InnerBorderOptions.Shrink();
	}
	
	return InnerBorderOptions;
}

void UWaveFunctionCollapseSubsystem::GatherInnerBorderOptionsToRemove(EWaveFunctionCollapseAdjacency Adjacency, const TArray<FWaveFunctionCollapseOption>& InitialOptions, TArray<FWaveFunctionCollapseOption>& OutBorderOptionsToRemove)
{
	bool bFoundBorderOptions = false;
	for (const FWaveFunctionCollapseOption& InitialOption : InitialOptions)
	{
		// if border option exists in the model, use it
		if (FWaveFunctionCollapseAdjacencyToOptionsMap* FoundBorderAdjacencyToOptionsMap = WFCModel->Constraints.Find(FWaveFunctionCollapseOption::BorderOption))
		{
			if (FWaveFunctionCollapseOptions* FoundBorderOptions = FoundBorderAdjacencyToOptionsMap->AdjacencyToOptionsMap.Find(Adjacency))
			{
				if (!FoundBorderOptions->Options.Contains(InitialOption))
				{
					OutBorderOptionsToRemove.AddUnique(InitialOption);
					bFoundBorderOptions = true;
				}
			}
		}
		
		// else, if useEmptyBorder, use empty option
		if (bUseEmptyBorder && !bFoundBorderOptions)
		{
			if (FWaveFunctionCollapseAdjacencyToOptionsMap* FoundEmptyAdjacencyToOptionsMap = WFCModel->Constraints.Find(FWaveFunctionCollapseOption::EmptyOption))
			{
				if (FWaveFunctionCollapseOptions* FoundEmptyOptions = FoundEmptyAdjacencyToOptionsMap->AdjacencyToOptionsMap.Find(Adjacency))
				{
					if (!FoundEmptyOptions->Options.Contains(InitialOption))
					{
						OutBorderOptionsToRemove.AddUnique(InitialOption);
					}
				}
			}
		}
	}
}

bool UWaveFunctionCollapseSubsystem::IsPositionInnerBorder(FIntVector Position)
{
	return (Position.X == 0
		|| Position.Y == 0
		|| Position.Z == 0
		|| Position.X == Resolution.X - 1
		|| Position.Y == Resolution.Y - 1
		|| Position.Z == Resolution.Z - 1);
}

bool UWaveFunctionCollapseSubsystem::Observe(TArray<FWaveFunctionCollapseTile>& Tiles, 
	TArray<int32>& RemainingTiles, 
	TMap<int32, FWaveFunctionCollapseQueueElement>& ObservationQueue,
	int32 RandomSeed)
{
	float MinEntropy = 0;
	int32 LastSameMinEntropyIndex = 0;
	int32 SelectedMinEntropyIndex = 0;
	int32 MinEntropyIndex = 0;
	FRandomStream RandomStream(RandomSeed);

	// Find MinEntropy Tile Indices
	if (RemainingTiles.Num() > 1)
	{
		for (int32 index = 0; index<RemainingTiles.Num(); index++)
		{
			if (index == 0)
			{
				MinEntropy = Tiles[RemainingTiles[index]].ShannonEntropy;
			}
			else
			{
				if (Tiles[RemainingTiles[index]].ShannonEntropy > MinEntropy)
				{
					break;
				}
				else
				{
					LastSameMinEntropyIndex += 1;
				}
			}
		}
		SelectedMinEntropyIndex = RandomStream.RandRange(0, LastSameMinEntropyIndex);
		MinEntropyIndex = RemainingTiles[SelectedMinEntropyIndex];
	}
	else
	{
		MinEntropyIndex = RemainingTiles[0];
	}

	// Rand Selection of Weighted Options using Cumulative Density
	TArray<float> CumulativeDensity;
	CumulativeDensity.Reserve(Tiles[MinEntropyIndex].RemainingOptions.Num());
	float CumulativeWeight = 0;
	for (FWaveFunctionCollapseOption& Option : Tiles[MinEntropyIndex].RemainingOptions)
	{
		CumulativeWeight += WFCModel->Constraints.Find(Option)->Weight;
		CumulativeDensity.Add(CumulativeWeight);
	}
	
	int32 SelectedOptionIndex = 0;
	float RandomDensity = RandomStream.FRandRange(0.0f, CumulativeDensity.Last());
	for (int32 Index = 0; Index < CumulativeDensity.Num(); Index++)
	{
		if (CumulativeDensity[Index] > RandomDensity)
		{
			SelectedOptionIndex = Index;
			break;
		}
	}

	// Make Selection
	Tiles[MinEntropyIndex] = FWaveFunctionCollapseTile(Tiles[MinEntropyIndex].RemainingOptions[SelectedOptionIndex], TNumericLimits<float>::Max());

	if (SelectedMinEntropyIndex != LastSameMinEntropyIndex)
	{
		RemainingTiles.Swap(SelectedMinEntropyIndex, LastSameMinEntropyIndex);
	}
	RemainingTiles.RemoveAtSwap(LastSameMinEntropyIndex);

	if (!RemainingTiles.IsEmpty())
	{
		// if MinEntropy has changed after removal, find new MinEntropy and swap to front of array
		if (Tiles[RemainingTiles[0]].ShannonEntropy != MinEntropy)
		{
			int32 SwapToIndex = 0;
			for (int32 index = 0; index < RemainingTiles.Num(); index++)
			{
				if (index == 0)
				{
					MinEntropy = Tiles[RemainingTiles[index]].ShannonEntropy;
				}
				else
				{
					if (Tiles[RemainingTiles[index]].ShannonEntropy < MinEntropy)
					{
						SwapToIndex = 0;
						MinEntropy = Tiles[RemainingTiles[index]].ShannonEntropy;
						RemainingTiles.Swap(SwapToIndex, index);
					}
					else if (Tiles[RemainingTiles[index]].ShannonEntropy == MinEntropy)
					{
						SwapToIndex += 1;
						RemainingTiles.Swap(SwapToIndex, index);
					}
				}
			}
		}

		// Add Adjacent Tile Indices to Queue
		AddAdjacentIndicesToQueue(MinEntropyIndex, RemainingTiles, ObservationQueue);

		// Continue To Propagation
		return true;
	}
	else
	{
		// Do Not Continue to Propagation
		return false;
	}
}

void UWaveFunctionCollapseSubsystem::AddAdjacentIndicesToQueue(int32 CenterIndex, const TArray<int32>& RemainingTiles, TMap<int32,FWaveFunctionCollapseQueueElement>& OutQueue)
{
	FIntVector Position = UWaveFunctionCollapseBPLibrary::IndexAsPosition(CenterIndex, Resolution);
	if (Position.X + 1 < Resolution.X && RemainingTiles.Contains(UWaveFunctionCollapseBPLibrary::PositionAsIndex(Position + FIntVector(1, 0, 0), Resolution)))
	{ 
		OutQueue.Add(UWaveFunctionCollapseBPLibrary::PositionAsIndex(Position + FIntVector(1, 0, 0), Resolution),
			FWaveFunctionCollapseQueueElement(CenterIndex, EWaveFunctionCollapseAdjacency::Front));
	}
	if (Position.X - 1 >= 0 && RemainingTiles.Contains(UWaveFunctionCollapseBPLibrary::PositionAsIndex(Position + FIntVector(-1, 0, 0), Resolution)))
	{ 
		OutQueue.Add(UWaveFunctionCollapseBPLibrary::PositionAsIndex(Position + FIntVector(-1, 0, 0), Resolution),
			FWaveFunctionCollapseQueueElement(CenterIndex, EWaveFunctionCollapseAdjacency::Back));
	}
	if (Position.Y + 1 < Resolution.Y && RemainingTiles.Contains(UWaveFunctionCollapseBPLibrary::PositionAsIndex(Position + FIntVector(0, 1, 0), Resolution)))
	{ 
		OutQueue.Add(UWaveFunctionCollapseBPLibrary::PositionAsIndex(Position + FIntVector(0, 1, 0), Resolution),
			FWaveFunctionCollapseQueueElement(CenterIndex, EWaveFunctionCollapseAdjacency::Right));
	}
	if (Position.Y - 1 >= 0 && RemainingTiles.Contains(UWaveFunctionCollapseBPLibrary::PositionAsIndex(Position + FIntVector(0, -1, 0), Resolution)))
	{ 
		OutQueue.Add(UWaveFunctionCollapseBPLibrary::PositionAsIndex(Position + FIntVector(0, -1, 0), Resolution),
			FWaveFunctionCollapseQueueElement(CenterIndex, EWaveFunctionCollapseAdjacency::Left));
	}
	if (Position.Z + 1 < Resolution.Z && RemainingTiles.Contains(UWaveFunctionCollapseBPLibrary::PositionAsIndex(Position + FIntVector(0, 0, 1), Resolution)))
	{ 
		OutQueue.Add(UWaveFunctionCollapseBPLibrary::PositionAsIndex(Position + FIntVector(0, 0, 1), Resolution),
			FWaveFunctionCollapseQueueElement(CenterIndex, EWaveFunctionCollapseAdjacency::Up));
	}
	if (Position.Z - 1 >= 0 && RemainingTiles.Contains(UWaveFunctionCollapseBPLibrary::PositionAsIndex(Position + FIntVector(0, 0, -1), Resolution)))
	{ 
		OutQueue.Add(UWaveFunctionCollapseBPLibrary::PositionAsIndex(Position + FIntVector(0, 0, -1), Resolution),
			FWaveFunctionCollapseQueueElement(CenterIndex, EWaveFunctionCollapseAdjacency::Down));
	}
}

bool UWaveFunctionCollapseSubsystem::Propagate(TArray<FWaveFunctionCollapseTile>& Tiles, 
	TArray<int32>& RemainingTiles, 
	TMap<int32, FWaveFunctionCollapseQueueElement>& ObservationQueue, 
	int32& PropagationCount)
{
	TMap<int32, FWaveFunctionCollapseQueueElement> PropagationQueue;

	while (!ObservationQueue.IsEmpty())
	{
		for (TPair<int32, FWaveFunctionCollapseQueueElement>& ObservationAdjacenctElement : ObservationQueue)
		{
			// Make sure the tile to check is still a valid remaining tile
			if (!RemainingTiles.Contains(ObservationAdjacenctElement.Key))
			{
				continue;
			}
			
			TArray<FWaveFunctionCollapseOption>& ObservationRemainingOptions = Tiles[ObservationAdjacenctElement.Key].RemainingOptions;

			// Get check against options
			TArray<FWaveFunctionCollapseOption> OptionsToCheckAgainst;
			OptionsToCheckAgainst.Reserve(WFCModel->Constraints.Num());
			for (FWaveFunctionCollapseOption& CenterOption : Tiles[ObservationAdjacenctElement.Value.CenterObjectIndex].RemainingOptions)
			{
				for (FWaveFunctionCollapseOption& Option : WFCModel->Constraints.FindRef(CenterOption).AdjacencyToOptionsMap.FindRef(ObservationAdjacenctElement.Value.Adjacency).Options)
				{
					OptionsToCheckAgainst.AddUnique(Option);
				}
			}
				
			// Accumulate New Remaining Options
			bool bAddToPropagationQueue = false;
			TArray<FWaveFunctionCollapseOption> TmpRemainingOptionsAccumulation;
			TmpRemainingOptionsAccumulation.Reserve(ObservationRemainingOptions.Num());
			for (FWaveFunctionCollapseOption& ObservationRemainingOption : ObservationRemainingOptions)
			{
				if (OptionsToCheckAgainst.Contains(ObservationRemainingOption))
				{
					TmpRemainingOptionsAccumulation.AddUnique(ObservationRemainingOption);
				}
				else
				{
					bAddToPropagationQueue = true;
				}
			}
				
			// If Remaining Options have changed
			if (bAddToPropagationQueue)
			{
				if (!TmpRemainingOptionsAccumulation.IsEmpty())
				{
					AddAdjacentIndicesToQueue(ObservationAdjacenctElement.Key, RemainingTiles, PropagationQueue);

					// Update Tile with new options
					float MinEntropy = Tiles[RemainingTiles[0]].ShannonEntropy;
					float NewEntropy = UWaveFunctionCollapseBPLibrary::CalculateShannonEntropy(TmpRemainingOptionsAccumulation, WFCModel);
					int32 CurrentRemainingTileIndex;
						
					// If NewEntropy is <= MinEntropy, add to front of Remaining Tiles
					if (NewEntropy < MinEntropy)
					{
						MinEntropy = NewEntropy;
						CurrentRemainingTileIndex = RemainingTiles.Find(ObservationAdjacenctElement.Key);
						if (CurrentRemainingTileIndex != 0)
						{
							RemainingTiles.Swap(0, CurrentRemainingTileIndex);
						}
					}
					else if (NewEntropy == MinEntropy)
					{
						CurrentRemainingTileIndex = RemainingTiles.Find(ObservationAdjacenctElement.Key);
						for (int32 Index = 1; Index < RemainingTiles.Num(); Index++)
						{
							if (MinEntropy != Tiles[RemainingTiles[Index]].ShannonEntropy)
							{
								if (CurrentRemainingTileIndex != Index)
								{
									RemainingTiles.Swap(Index, CurrentRemainingTileIndex);
								}
								break;
							}
						}
					}
						
					Tiles[ObservationAdjacenctElement.Key] = FWaveFunctionCollapseTile(TmpRemainingOptionsAccumulation, NewEntropy);
				}
				else
				{
					// Encountered Contradiction
					UE_LOGF(LogWFC, Error, "Encountered Contradiction on Index %d", ObservationAdjacenctElement.Key);
					return false;
				}
			}
		}

		ObservationQueue = PropagationQueue;
		if (!PropagationQueue.IsEmpty())
		{
			PropagationCount += 1;
			PropagationQueue.Reset();
		}
	}

	return true;
}

bool UWaveFunctionCollapseSubsystem::ObservationPropagation(TArray<FWaveFunctionCollapseTile>& Tiles, 
	TArray<int32>& RemainingTiles,
	TMap<int32, FWaveFunctionCollapseQueueElement>& ObservationQueue,
	int32 RandomSeed)
{
	int32 PropagationCount = 1;
	int32 MutatedRandomSeed = RandomSeed;
	
	while (Observe(Tiles, RemainingTiles, ObservationQueue, MutatedRandomSeed))
	{
		if (!Propagate(Tiles, RemainingTiles, ObservationQueue, PropagationCount))
		{
			return false;
		}

		// Mutate Seed
		MutatedRandomSeed--;
	}

	// Check if all tiles in the solve are non-spawnable
	return !AreAllTilesNonSpawnable(Tiles);
}

UActorComponent* UWaveFunctionCollapseSubsystem::AddNamedInstanceComponent(AActor* Actor, TSubclassOf<UActorComponent> ComponentClass, FName ComponentName)
{
	Actor->Modify();
	// Assign Unique Name
	int32 Counter = 1;
	FName ComponentInstanceName = ComponentName;
	while (!FComponentEditorUtils::IsComponentNameAvailable(ComponentInstanceName.ToString(), Actor))
	{
		ComponentInstanceName = FName(*FString::Printf(TEXT("%s_%d"), *ComponentName.ToString(), Counter++));
	}
	UActorComponent* InstanceComponent = NewObject<UActorComponent>(Actor, ComponentClass, ComponentInstanceName, RF_Transactional);
	if (InstanceComponent)
	{
		Actor->AddInstanceComponent(InstanceComponent);
		Actor->FinishAddComponent(InstanceComponent, false, FTransform::Identity);
		Actor->RerunConstructionScripts();
	}
	return InstanceComponent;
}

AActor* UWaveFunctionCollapseSubsystem::SpawnActorFromTiles(const TArray<FWaveFunctionCollapseTile>& Tiles)
{
	UEditorActorSubsystem* EditorActorSubsystem = GEditor->GetEditorSubsystem<UEditorActorSubsystem>();
	// Spawn Actor
	AActor* SpawnedActor = EditorActorSubsystem->SpawnActorFromClass(AActor::StaticClass(), FVector::ZeroVector);
	SpawnedActor->GetRootComponent()->SetMobility(EComponentMobility::Static);
	SpawnedActor->SetActorLocationAndRotation(OriginLocation, Orientation);
	FActorLabelUtilities::SetActorLabelUnique(SpawnedActor, WFCModel->GetFName().ToString());

	// Create Components
	TMap<FSoftObjectPath, UInstancedStaticMeshComponent*> BaseObjectToISM;
	for (int32 index = 0; index < Tiles.Num(); index++)
	{
		if (Tiles[index].RemainingOptions.Num() != 1)
		{
			continue;
		}

		// check for empty or void options
		FSoftObjectPath BaseObject = Tiles[index].RemainingOptions[0].BaseObject;
		if (BaseObject == FWaveFunctionCollapseOption::EmptyOption.BaseObject
			|| BaseObject == FWaveFunctionCollapseOption::VoidOption.BaseObject)
		{
			continue;
		}

		// Check the SpawnExclusion array
		if (WFCModel->SpawnExclusion.Contains(BaseObject))
		{
			continue;
		}

		UObject* LoadedObject = BaseObject.TryLoad();
		if (LoadedObject)
		{
			FRotator BaseRotator = Tiles[index].RemainingOptions[0].BaseRotator;
			FVector BaseScale3D = Tiles[index].RemainingOptions[0].BaseScale3D;
			FVector PositionOffset = FVector(WFCModel->TileSize * 0.5f);
			FVector TilePosition = (FVector(UWaveFunctionCollapseBPLibrary::IndexAsPosition(index, Resolution)) * WFCModel->TileSize) + PositionOffset;

			// Static meshes are handled with ISM Components
			if (UStaticMesh* LoadedStaticMesh = Cast<UStaticMesh>(LoadedObject))
			{
				UInstancedStaticMeshComponent* ISMComponent;
				if (UInstancedStaticMeshComponent** FoundISMComponentPtr = BaseObjectToISM.Find(BaseObject))
				{
					ISMComponent = *FoundISMComponentPtr;
				}
				else
				{
					ISMComponent = Cast<UInstancedStaticMeshComponent>(UWaveFunctionCollapseSubsystem::AddNamedInstanceComponent(SpawnedActor, UInstancedStaticMeshComponent::StaticClass(), LoadedObject->GetFName()));
					BaseObjectToISM.Add(BaseObject, ISMComponent);
				}
				ISMComponent->SetStaticMesh(LoadedStaticMesh);
				ISMComponent->SetMobility(EComponentMobility::Static);
				ISMComponent->AddInstance(FTransform(BaseRotator, TilePosition, BaseScale3D));
			}
			// Blueprints are handled with ChildActorComponents
			else if (UBlueprint* LoadedBlueprint = Cast<UBlueprint>(LoadedObject))
			{
				const TSubclassOf<UObject> GeneratedClass = LoadedBlueprint->GeneratedClass;

				// if BP is placeable
				if (!(GeneratedClass->GetClass()->HasAnyClassFlags(CLASS_NotPlaceable | CLASS_Deprecated | CLASS_Abstract))
					&& GeneratedClass->GetClass()->IsChildOf(AActor::StaticClass()))
				{
					UChildActorComponent* ChildActorComponent = Cast<UChildActorComponent>(UWaveFunctionCollapseSubsystem::AddNamedInstanceComponent(SpawnedActor, UChildActorComponent::StaticClass(), LoadedObject->GetFName()));
					ChildActorComponent->SetChildActorClass(*GeneratedClass);
					ChildActorComponent->SetRelativeLocation(TilePosition);
					ChildActorComponent->SetRelativeRotation(BaseRotator);
					ChildActorComponent->SetRelativeScale3D(BaseScale3D);
				}
			}
			else
			{
				UE_LOGF(LogWFC, Warning, "Invalid Type, skipping: %ls", *BaseObject.ToString());
			}
		}
		else
		{
			UE_LOGF(LogWFC, Warning, "Unable to load object, skipping: %ls", *BaseObject.ToString());
		}
	}

	return SpawnedActor;
}

bool UWaveFunctionCollapseSubsystem::AreAllTilesNonSpawnable(const TArray<FWaveFunctionCollapseTile>& Tiles)
{
	bool bAllTilesAreNonSpawnable = true;
	for (int32 index = 0; index < Tiles.Num(); index++)
	{
		if (Tiles[index].RemainingOptions.Num() == 1)
		{
			FSoftObjectPath BaseObject = Tiles[index].RemainingOptions[0].BaseObject;
			if (!(BaseObject == FWaveFunctionCollapseOption::EmptyOption.BaseObject
				|| BaseObject == FWaveFunctionCollapseOption::VoidOption.BaseObject
				|| WFCModel->SpawnExclusion.Contains(BaseObject)))
			{
				bAllTilesAreNonSpawnable = false;
				break;
			}
		}
	}
	return bAllTilesAreNonSpawnable;
}

void UWaveFunctionCollapseSubsystem::DeriveGridFromTransformBounds(const TArray<FTransform>& Transforms)
{
	if (Transforms.IsEmpty())
	{
		UE_LOGF(LogWFC, Error, "Empty Transform Array.");
		return;
	}

	FVector TransformPivot;
	FVector AxisAlignedPoint = FVector::ZeroVector;
	FVector MinBound = FVector::ZeroVector;
	FVector MaxBound = FVector::ZeroVector;
	TMap<FIntVector, int32> PositionToPointCount;
	float TransformOrientation = Transforms[0].GetRotation().Rotator().Yaw;

	float CosOrientation = FMath::Cos(PI / (180.f) * -TransformOrientation);
	float SinOrientation = FMath::Sin(PI / (180.f) * -TransformOrientation);
	float UnitScale = 1.0f;

	for (int32 index = 0; index < Transforms.Num(); index++)
	{
		if (index == 0)
		{
			TransformPivot = Transforms[index].GetLocation() * UnitScale;
		}
		else
		{
			FVector OffsetPoint = Transforms[index].GetLocation() * UnitScale - TransformPivot;
			AxisAlignedPoint.X = (OffsetPoint.X * CosOrientation) - (OffsetPoint.Y * SinOrientation);
			AxisAlignedPoint.Y = (OffsetPoint.X * SinOrientation) + (OffsetPoint.Y * CosOrientation);
			AxisAlignedPoint.Z = OffsetPoint.Z;
			MinBound.X = FMath::Min(AxisAlignedPoint.X, MinBound.X);
			MinBound.Y = FMath::Min(AxisAlignedPoint.Y, MinBound.Y);
			MinBound.Z = FMath::Min(AxisAlignedPoint.Z, MinBound.Z);
			MaxBound.X = FMath::Max(AxisAlignedPoint.X, MaxBound.X);
			MaxBound.Y = FMath::Max(AxisAlignedPoint.Y, MaxBound.Y);
			MaxBound.Z = FMath::Max(AxisAlignedPoint.Z, MaxBound.Z);
		}
		
		FIntVector PointPosition;
		PointPosition.X = (int32)FMath::RoundHalfFromZero(AxisAlignedPoint.X / WFCModel->TileSize);
		PointPosition.Y = (int32)FMath::RoundHalfFromZero(AxisAlignedPoint.Y / WFCModel->TileSize);
		PointPosition.Z = (int32)FMath::RoundHalfFromZero(AxisAlignedPoint.Z / WFCModel->TileSize);
		if (int32* PointCount = PositionToPointCount.Find(PointPosition))
		{
			PositionToPointCount.Add(PointPosition, *PointCount + 1);
		}
		else
		{
			PositionToPointCount.Add(PointPosition, 1);
		}
	}

	// Set WFC Resolution
	Resolution.X = (int32)FMath::RoundHalfFromZero(MaxBound.X / WFCModel->TileSize) - (int32)FMath::RoundHalfFromZero(MinBound.X / WFCModel->TileSize) + 1;
	Resolution.Y = (int32)FMath::RoundHalfFromZero(MaxBound.Y / WFCModel->TileSize) - (int32)FMath::RoundHalfFromZero(MinBound.Y / WFCModel->TileSize) + 1;
	Resolution.Z = (int32)FMath::RoundHalfFromZero(MaxBound.Z / WFCModel->TileSize) - (int32)FMath::RoundHalfFromZero(MinBound.Z / WFCModel->TileSize) + 1;

	// Set WFC OriginLocation
	FVector ReorientedMinPoint;
	ReorientedMinPoint.X = ((MinBound.X - (WFCModel->TileSize * 0.5f)) * CosOrientation) - ((MinBound.Y - (WFCModel->TileSize * 0.5f)) * -SinOrientation);
	ReorientedMinPoint.Y = ((MinBound.X - (WFCModel->TileSize * 0.5f)) * -SinOrientation) + ((MinBound.Y - (WFCModel->TileSize * 0.5f)) * CosOrientation);
	ReorientedMinPoint.Z = MinBound.Z - (WFCModel->TileSize * 0.5f);
	OriginLocation = ReorientedMinPoint + TransformPivot;

	// Set WFC Orientation
	Orientation = FRotator(0, TransformOrientation, 0);

	// Set WFC Starter Options
	FIntVector PositionOffset;
	PositionOffset.X = (int32)FMath::RoundHalfFromZero(MinBound.X / WFCModel->TileSize);
	PositionOffset.Y = (int32)FMath::RoundHalfFromZero(MinBound.Y / WFCModel->TileSize);
	PositionOffset.Z = (int32)FMath::RoundHalfFromZero(MinBound.Z / WFCModel->TileSize);
	StarterOptions.Empty();
	for (int32 z = 0;z < Resolution.Z; z++)
	{
		for (int32 y = 0;y < Resolution.Y; y++)
		{
			for (int32 x = 0;x < Resolution.X; x++)
			{
				if (!PositionToPointCount.Contains(FIntVector(x + PositionOffset.X, y + PositionOffset.Y, z + PositionOffset.Z)))
				{
					StarterOptions.Add(FIntVector(x,y,z), FWaveFunctionCollapseOption::EmptyOption);
				}
			}
		}
	}
}

void UWaveFunctionCollapseSubsystem::DeriveGridFromTransforms(const TArray<FTransform>& Transforms)
{
	if (Transforms.IsEmpty())
	{
		UE_LOGF(LogWFC, Error, "Empty Transform Array.");
		return;
	}

	FVector TransformPivot;
	FVector AxisAlignedPoint = FVector::ZeroVector;
	FVector MinBound = FVector::ZeroVector;
	FVector MaxBound = FVector::ZeroVector;
	TArray<FIntVector> PointPositions;
	float TransformOrientation = Transforms[0].GetRotation().Rotator().Yaw;

	float CosOrientation = FMath::Cos(PI / (180.f) * -TransformOrientation);
	float SinOrientation = FMath::Sin(PI / (180.f) * -TransformOrientation);
	float UnitScale = 1.0f;
	
	for (int32 index = 0; index < Transforms.Num(); index++)
	{
		if (index == 0)
		{
			TransformPivot = (Transforms[index].GetLocation() * UnitScale);
		}
		else
		{
			FVector LocalPoint = Transforms[index].GetLocation() * UnitScale - TransformPivot;
			AxisAlignedPoint.X = (LocalPoint.X * CosOrientation) - (LocalPoint.Y * SinOrientation);
			AxisAlignedPoint.Y = (LocalPoint.X * SinOrientation) + (LocalPoint.Y * CosOrientation);
			AxisAlignedPoint.Z = LocalPoint.Z;
			MinBound.X = FMath::Min(AxisAlignedPoint.X, MinBound.X);
			MinBound.Y = FMath::Min(AxisAlignedPoint.Y, MinBound.Y);
			MinBound.Z = FMath::Min(AxisAlignedPoint.Z, MinBound.Z);
			MaxBound.X = FMath::Max(AxisAlignedPoint.X, MaxBound.X);
			MaxBound.Y = FMath::Max(AxisAlignedPoint.Y, MaxBound.Y);
			MaxBound.Z = FMath::Max(AxisAlignedPoint.Z, MaxBound.Z);
		}
		
		FIntVector PointPosition;
		PointPosition.X = (int32)FMath::RoundHalfFromZero(AxisAlignedPoint.X / WFCModel->TileSize);
		PointPosition.Y = (int32)FMath::RoundHalfFromZero(AxisAlignedPoint.Y / WFCModel->TileSize);
		PointPosition.Z = (int32)FMath::RoundHalfFromZero(AxisAlignedPoint.Z / WFCModel->TileSize);
		PointPositions.AddUnique(PointPosition);
	}

	// Set WFC Resolution
	Resolution.X = (int32)FMath::RoundHalfFromZero(MaxBound.X / WFCModel->TileSize) - (int32)FMath::RoundHalfFromZero(MinBound.X / WFCModel->TileSize) + 1;
	Resolution.Y = (int32)FMath::RoundHalfFromZero(MaxBound.Y / WFCModel->TileSize) - (int32)FMath::RoundHalfFromZero(MinBound.Y / WFCModel->TileSize) + 1;
	Resolution.Z = (int32)FMath::RoundHalfFromZero(MaxBound.Z / WFCModel->TileSize) - (int32)FMath::RoundHalfFromZero(MinBound.Z / WFCModel->TileSize) + 1;

	// Set WFC OriginLocation
	FVector ReorientedMinPoint;
	ReorientedMinPoint.X = ((MinBound.X - (WFCModel->TileSize * 0.5f)) * CosOrientation) - ((MinBound.Y - (WFCModel->TileSize * 0.5f)) * -SinOrientation);
	ReorientedMinPoint.Y = ((MinBound.X - (WFCModel->TileSize * 0.5f)) * -SinOrientation) + ((MinBound.Y - (WFCModel->TileSize * 0.5f)) * CosOrientation);
	ReorientedMinPoint.Z = MinBound.Z - (WFCModel->TileSize * 0.5f);
	OriginLocation = ReorientedMinPoint + TransformPivot;
	
	// Set WFC Orientation
	Orientation = FRotator(0, TransformOrientation, 0);

	// Set WFC Starter Options
	FIntVector PositionOffset;
	PositionOffset.X = (int32)FMath::RoundHalfFromZero(MinBound.X / WFCModel->TileSize);
	PositionOffset.Y = (int32)FMath::RoundHalfFromZero(MinBound.Y / WFCModel->TileSize);
	PositionOffset.Z = (int32)FMath::RoundHalfFromZero(MinBound.Z / WFCModel->TileSize);
	StarterOptions.Empty();
	for (int32 z = 0;z < Resolution.Z; z++)
	{
		for (int32 y = 0;y < Resolution.Y; y++)
		{
			for (int32 x = 0;x < Resolution.X; x++)
			{
				if (!PointPositions.Contains(FIntVector(x + PositionOffset.X, y + PositionOffset.Y, z + PositionOffset.Z)))
				{
					StarterOptions.Add(FIntVector(x, y, z), FWaveFunctionCollapseOption::EmptyOption);
				}
			}
		}
	}
	for (TPair<FIntVector,FWaveFunctionCollapseOption> StarterOption : StarterOptions)
	{
		UE_LOGF(LogWFC, Display, "StartOption at: %d,%d,%d", StarterOption.Key.X, StarterOption.Key.Y, StarterOption.Key.Z);
	}
}
