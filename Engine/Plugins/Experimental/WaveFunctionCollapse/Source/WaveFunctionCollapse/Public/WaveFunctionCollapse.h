// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

class FMenuBuilder;
class FReply;

class FWaveFunctionCollapseModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:

	void AddMenuEntry(FMenuBuilder& MenuBuilder);
	void WaveFunctionCollapseUI();

	TSharedPtr<class FUICommandList> PluginCommands;
};