// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "WaveFunctionCollapseModel.h"
#include "WaveFunctionCollapseBPLibrary.generated.h"

USTRUCT(BlueprintType)
struct WAVEFUNCTIONCOLLAPSE_API FWaveFunctionCollapseNeighbor
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse")
	EWaveFunctionCollapseAdjacency Adjacency = EWaveFunctionCollapseAdjacency::Front;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse")
	int32 Step = 1;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse", meta = (AllowedClasses = "/Script/Engine.StaticMesh, /Script/Engine.Blueprint"))
	FSoftObjectPath NeighborObject;
};

USTRUCT(BlueprintType)
struct WAVEFUNCTIONCOLLAPSE_API FWaveFunctionCollapseNeighborRule
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse", meta = (AllowedClasses = "/Script/Engine.StaticMesh, /Script/Engine.Blueprint"))
	FSoftObjectPath KeyObject;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse")
	TArray<FWaveFunctionCollapseNeighbor> Neighbors;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse", meta = (AllowedClasses = "/Script/Engine.StaticMesh, /Script/Engine.Blueprint"))
	TArray<FSoftObjectPath> SpawnObjects;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse")
	FTransform SpawnRelativeTransform;

	/**
	* SpawnChance 1 = 100 % chance to spawn, SpawnChance 0 = 0 % chance to spawn
	*/ 
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse", meta = (ClampMin = "0", ClampMax = "1"))
	float SpawnChance = 1.0f;
};

UCLASS()
class WAVEFUNCTIONCOLLAPSE_API UWaveFunctionCollapseBPLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/**
	* Calculates Shannon Entropy from an array of options and a given model
	* @param Options Array of options
	* @param WFCModel WaveFunctionCollapseModel that stores weights for options 
	*/
	UFUNCTION(BlueprintCallable, Category = "WaveFunctionCollapse")
	static float CalculateShannonEntropy(const TArray<FWaveFunctionCollapseOption>& Options, UWaveFunctionCollapseModel* WFCModel);

	/**
	* Convert 3D grid position to 2D array index
	* @param Position 3D grid position
	* @param Resolution Grid resolution
	* @return 2D array index
	*/
	UFUNCTION(BlueprintPure, Category = "WaveFunctionCollapse")
	static int32 PositionAsIndex(FIntVector Position, FIntVector Resolution);

	/**
	* Convert 2D array index to 3D grid position
	* @param Index 2D array index
	* @param Resolution
	* @return 3D grid position
	*/
	UFUNCTION(BlueprintPure, Category = "WaveFunctionCollapse")
	static FIntVector IndexAsPosition(int32 Index, FIntVector Resolution);

	/**
	* Builds the initial tile which adds every unique option in a model to its RemainingOptions array and calculates its entropy
	* @param WFCModel WaveFunctionCollapseModel
	*/
	UFUNCTION(BlueprintCallable, Category = "WaveFunctionCollapse", meta = (ScriptMethod))
	static FWaveFunctionCollapseTile BuildInitialTile(UWaveFunctionCollapseModel* WFCModel);

	/**
	* Get adjacent indices of a given index and its adjacency
	* @param Index 2D array index
	* @param Resolution
	* @return Map of valid adjacent indices and its adjacency in relation to the given index
	*/
	UFUNCTION(BlueprintCallable, Category = "WaveFunctionCollapse")
	static TMap<int32, EWaveFunctionCollapseAdjacency> GetAdjacentIndices(int32 Index, FIntVector Resolution);

	/**
	* Get adjacent positions of a given position and its adjacency
	* @param Position 3D grid position
	* @param Resolution Grid resolution
	* @return Map of valid adjacent positions and its adjacency in relation to the given position
	*/
	UFUNCTION(BlueprintCallable, Category = "WaveFunctionCollapse")
	static TMap<FIntVector, EWaveFunctionCollapseAdjacency> GetAdjacentPositions(FIntVector Position, FIntVector Resolution);

	/**
	* Is the option contained in the given options array
	* @param Option Option to check for
	* @param Options Array of options
	*/
	UFUNCTION(BlueprintPure, Category = "WaveFunctionCollapse")
	static bool IsOptionContained(const FWaveFunctionCollapseOption& Option, const TArray<FWaveFunctionCollapseOption>& Options);

	/**
	* Get the opposite adjacency for a given adjacency.  For example GetOppositeAdjacency(Front) will return Back.
	* @param Adjacency Adjacency direction
	*/
	UFUNCTION(BlueprintPure, Category = "WaveFunctionCollapse")
	static EWaveFunctionCollapseAdjacency GetOppositeAdjacency(EWaveFunctionCollapseAdjacency Adjacency);
	
	/**
	* Get the next adjacency in clockwise direction on a Z-axis for a given adjacency.
	* For example GetNextZAxisClockwiseAdjacency(Front) will return Right.
	* Up or Down will return the original adjacency.
	* @param Adjacency Adjacency direction
	*/
	UFUNCTION(BlueprintPure, Category = "WaveFunctionCollapse")
	static EWaveFunctionCollapseAdjacency GetNextZAxisClockwiseAdjacency(EWaveFunctionCollapseAdjacency Adjacency);

	/**
	* Add an entry to an AdjacencyToOptionsMap
	*/
	UFUNCTION(BlueprintCallable, Category = "WaveFunctionCollapse")
	static void AddToAdjacencyToOptionsMap(UPARAM(ref) FWaveFunctionCollapseAdjacencyToOptionsMap& AdjacencyToOptionsMap, EWaveFunctionCollapseAdjacency Adjacency, FWaveFunctionCollapseOption Option);

	UFUNCTION(BlueprintPure, Category = "WaveFunctionCollapse")
	static bool IsSoftObjPathEqual(const FSoftObjectPath& SoftObjectPathA, const FSoftObjectPath& SoftObjectPathB);

	/**
	* Converts Rotator to Matrix and back to sanitize multiple representations of the same rotation to a common Rotator value
	*/
	UFUNCTION(BlueprintPure, Category = "WaveFunctionCollapse")
	static FRotator SanitizeRotator(FRotator Rotator);

	/**
	* Derive constraints from a given layout of actors and append them to a model
	* @param Actors array of actors to evaluate
	* @param WFCModel to add constraints to
	* @param TileSize distance between tiles
	* @param bIsBorderEmptyOption should the border be considered EmptyOption
	* @param bIsMinZFloorOption should the minimum Z actors be considered floor options (nothing can go below it)
	* @param bAutoDeriveZAxisRotationConstraints should it auto derive z-axis rotational variants
	* @param SpawnExclusionAssets assets to exclude when spawning
	* @param IgnoreRotationAssets assets to ignore rotation variants
	*/
	UFUNCTION(BlueprintCallable, Category = "WaveFunctionCollapse")
	static void DeriveModelFromActors(UPARAM(ref) const TArray<AActor*>& Actors, 
		UWaveFunctionCollapseModel* WFCModel, 
		float TileSize, 
		bool bIsBorderEmptyOption, 
		bool bIsMinZFloorOption, 
		bool bUseUniformWeightDistribution,
		bool bAutoDeriveZAxisRotationConstraints,
		const TArray<FSoftObjectPath>& SpawnExclusionAssets,
		const TArray<FSoftObjectPath>& IgnoreRotationAssets);

	/**
	* Get PositionToOptionsMap from a given actor that has ISM components.
	* This is useful when you want to derive neighboring tile data from a WFC-solved actor to be used for post processing.
	* This will only evaluate ISM components.
	* @param Actor Actor with ISM components
	* @param TileSize distance between tiles
	* @param PositionToOptionMap 
	*/
	UFUNCTION(BlueprintCallable, Category = "WaveFunctionCollapse")
	static bool GetPositionToOptionMapFromActor(AActor* Actor, float TileSize, UPARAM(ref) TMap<FIntVector, FWaveFunctionCollapseOption>& PositionToOptionMap);

	/**
	* Make FWaveFunctionCollapseOption of type: EmptyOption
	*/
	UFUNCTION(BlueprintPure, Category = "WaveFunctionCollapse")
	static FWaveFunctionCollapseOption MakeEmptyOption();

	/**
	* Make FWaveFunctionCollapseOption of type: BorderOption
	*/
	UFUNCTION(BlueprintPure, Category = "WaveFunctionCollapse")
	static FWaveFunctionCollapseOption MakeBorderOption();
	
	/**
	* Make FWaveFunctionCollapseOption of type: VoidOption
	*/
	UFUNCTION(BlueprintPure, Category = "WaveFunctionCollapse")
	static FWaveFunctionCollapseOption MakeVoidOption();
};
