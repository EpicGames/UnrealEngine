// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "Engine/StaticMesh.h"
#include "WaveFunctionCollapseModel.generated.h"

UENUM(BlueprintType)
enum class EWaveFunctionCollapseAdjacency : uint8
{
	Front	UMETA(DisplayName = "X+ Front"),
	Back	UMETA(DisplayName = "X- Back"),
	Right	UMETA(DisplayName = "Y+ Right"),
	Left	UMETA(DisplayName = "Y- Left"),
	Up		UMETA(DisplayName = "Z+ Up"),
	Down	UMETA(DisplayName = "Z- Down")
};

/**
* Base Option Struct which holds an object, its orientation and scale
*/
USTRUCT(BlueprintType)
struct WAVEFUNCTIONCOLLAPSE_API FWaveFunctionCollapseOption
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse", meta = (AllowedClasses = "/Script/Engine.StaticMesh, /Script/Engine.Blueprint"))
	FSoftObjectPath BaseObject;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse")
	FRotator BaseRotator = FRotator::ZeroRotator;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse")
	FVector BaseScale3D = FVector::OneVector;

	FWaveFunctionCollapseOption() = default;

	FWaveFunctionCollapseOption(FString Object, FRotator Rotator, FVector Scale3d)
	{
		BaseObject = FSoftObjectPath(Object);
		BaseRotator = Rotator;
		BaseScale3D = Scale3d;
	}

	FWaveFunctionCollapseOption(FString Object)
	{
		BaseObject = FSoftObjectPath(Object);
	}

	static const FWaveFunctionCollapseOption EmptyOption;
	static const FWaveFunctionCollapseOption BorderOption;
	static const FWaveFunctionCollapseOption VoidOption;

	friend uint32 GetTypeHash(FWaveFunctionCollapseOption Output)
	{
		uint32 OutputHash;
		OutputHash = HashCombine(GetTypeHash(Output.BaseRotator.Vector()), GetTypeHash(Output.BaseScale3D));
		OutputHash = HashCombine(OutputHash, GetTypeHash(Output.BaseObject));
		return OutputHash;
	}

	bool operator==(const FWaveFunctionCollapseOption& Rhs) const
	{
		return BaseObject == Rhs.BaseObject && BaseRotator.Equals(Rhs.BaseRotator) && BaseScale3D.Equals(Rhs.BaseScale3D);
	}

	bool operator!=(const FWaveFunctionCollapseOption& Rhs) const
	{
		return BaseObject != Rhs.BaseObject || !BaseRotator.Equals(Rhs.BaseRotator) || !BaseScale3D.Equals(Rhs.BaseScale3D);
	}

	bool IsBaseObject(FString ObjectPath)
	{
		return (BaseObject == FSoftObjectPath(ObjectPath));
	}
};

/**
* Container struct for array of Options
*/
USTRUCT(BlueprintType)
struct WAVEFUNCTIONCOLLAPSE_API FWaveFunctionCollapseOptions
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse")
	TArray<FWaveFunctionCollapseOption> Options;
};

/**
* Container struct for AdjacencyToOptionsMap
* Stores the weight and contribution of an option
*/
USTRUCT(BlueprintType)
struct WAVEFUNCTIONCOLLAPSE_API FWaveFunctionCollapseAdjacencyToOptionsMap
{
	GENERATED_BODY()

	/**
	* The amount of times an option is present when deriving a model.
	* This value is used to calculate its weight.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse", meta = (ClampMin = "0"))
	int32 Contribution = 1;

	/**
	* The weight of an option calculated by dividing this Contribution by the sum of all contributions of all options.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse", meta = (ClampMin = "0", ClampMax = "1"))
	float Weight;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse")
	TMap<EWaveFunctionCollapseAdjacency, FWaveFunctionCollapseOptions> AdjacencyToOptionsMap;
};

/**
* A Model of WFC constraints.
* This data asset should contain all necessary data to allow for a WFC solve of an arbitrary grid size.
*/
UCLASS(Blueprintable, BlueprintType)
class WAVEFUNCTIONCOLLAPSE_API UWaveFunctionCollapseModel : public UDataAsset
{
	GENERATED_BODY()

public:
	/**
	* Grid Tile Size in cm^3
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse")
	float TileSize;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse")
	TMap<FWaveFunctionCollapseOption, FWaveFunctionCollapseAdjacencyToOptionsMap> Constraints;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse", meta = (AllowedClasses = "/Script/Engine.StaticMesh, /Script/Engine.Blueprint"))
	TArray<FSoftObjectPath> SpawnExclusion;

	/**
	* Create a constraint
	* @param KeyOption Key option
	* @param Adjacency Adjacency from KeyOption to AdjacentOption
	* @param AdjacentOption Adjacent option
	*/
	UFUNCTION(BlueprintCallable, Category = "WaveFunctionCollapse")
	void AddConstraint(const FWaveFunctionCollapseOption& KeyOption, EWaveFunctionCollapseAdjacency Adjacency, const FWaveFunctionCollapseOption& AdjacentOption);

	/**
	* Get all options for a given key option in a given adjacency
	* @param KeyOption Key option
	* @param Adjacency Adjacency from KeyOption to AdjacentOption
	*/
	UFUNCTION(BlueprintCallable, Category = "WaveFunctionCollapse")
	FWaveFunctionCollapseOptions GetOptions(const FWaveFunctionCollapseOption& KeyOption, EWaveFunctionCollapseAdjacency Adjacency) const;

	/**
	* Set the weights of key objects based on their contribution values
	*/
	UFUNCTION(BlueprintCallable, Category = "WaveFunctionCollapse")
	void SetWeightsFromContributions();

	/**
	* Set the weights of key objects to a given value
	*/
	UFUNCTION(BlueprintCallable, Category = "WaveFunctionCollapse")
	void SetAllWeights(float Weight);

	/**
	* Set the contribution values of key objects to a given value
	*/
	UFUNCTION(BlueprintCallable, Category = "WaveFunctionCollapse")
	void SetAllContributions(int32 Contribution);

	/**
	* Set the contribution value of a key object to a given value
	*/
	UFUNCTION(BlueprintCallable, Category = "WaveFunctionCollapse")
	void SetOptionContribution(const FWaveFunctionCollapseOption& Option, int32 Contribution);

	/**
	* Get the weight value of an option
	*/
	UFUNCTION(BlueprintCallable, Category = "WaveFunctionCollapse")
	float GetOptionWeight(const FWaveFunctionCollapseOption& Option) const;

	/**
	* Get the contribution value of an option
	*/
	UFUNCTION(BlueprintCallable, Category = "WaveFunctionCollapse")
	int32 GetOptionContribution(const FWaveFunctionCollapseOption& Option) const;

	/**
	* Get the total count of constraints in this model
	*/
	UFUNCTION(BlueprintCallable, Category = "WaveFunctionCollapse")
	int32 GetConstraintCount() const;

	/**
	* Swap meshes in the model with other meshes based on a map.  
	* This is useful when working with template meshes that need to be swapped.
	*/
	UFUNCTION(BlueprintCallable, Category = "WaveFunctionCollapse")
	void SwapMeshes(TMap<UStaticMesh*, UStaticMesh*> SourceToTargetMeshMap);
};

/**
* Base Tile Struct which holds an array of remaining Options and its Shannon Entropy value
*/
USTRUCT(BlueprintType)
struct WAVEFUNCTIONCOLLAPSE_API FWaveFunctionCollapseTile
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse")
	float ShannonEntropy;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse")
	TArray<FWaveFunctionCollapseOption> RemainingOptions;

	FWaveFunctionCollapseTile() = default;

	FWaveFunctionCollapseTile(const TArray<FWaveFunctionCollapseOption>& Options, float Entropy)
	{
		RemainingOptions = Options;
		ShannonEntropy = Entropy;
	}

	// constructor with only one option
	FWaveFunctionCollapseTile(const FWaveFunctionCollapseOption& Option, float Entropy)
	{
		RemainingOptions.Add(Option);
		ShannonEntropy = Entropy;
	}
};

/**
* A helper struct used for queuing during Observation and Propagation phases
*/
USTRUCT(BlueprintType)
struct WAVEFUNCTIONCOLLAPSE_API FWaveFunctionCollapseQueueElement
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse")
	int32 CenterObjectIndex;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "WaveFunctionCollapse")
	EWaveFunctionCollapseAdjacency Adjacency = EWaveFunctionCollapseAdjacency::Front;

	FWaveFunctionCollapseQueueElement() = default;

	FWaveFunctionCollapseQueueElement(int32 CenterObjectIndexInput, EWaveFunctionCollapseAdjacency AdjacencyInput)
	{
		CenterObjectIndex = CenterObjectIndexInput;
		Adjacency = AdjacencyInput;
	}
};