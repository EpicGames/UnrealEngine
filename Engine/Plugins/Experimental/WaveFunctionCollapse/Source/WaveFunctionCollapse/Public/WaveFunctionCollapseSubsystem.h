// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "WaveFunctionCollapseModel.h"
#include "Components/ActorComponent.h"
#include "WaveFunctionCollapseSubsystem.generated.h"

WAVEFUNCTIONCOLLAPSE_API DECLARE_LOG_CATEGORY_EXTERN(LogWFC, Log, All);

UCLASS()
class WAVEFUNCTIONCOLLAPSE_API UWaveFunctionCollapseSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "WFCSettings")
	TObjectPtr<UWaveFunctionCollapseModel> WFCModel;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "WFCSettings")
	FIntVector Resolution;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "WFCSettings")
	FVector OriginLocation = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "WFCSettings")
	FRotator Orientation = FRotator::ZeroRotator;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "WFCSettings")
	bool bUseEmptyBorder;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "WFCSettings")
	TMap<FIntVector, FWaveFunctionCollapseOption> StarterOptions;

	/**
	* Solve a grid using a WFC model.  If successful, spawn an actor.
	* @param TryCount Amount of times to attempt a successful solve
	* @param RandomSeed Seed for deterministic results.  When this value is 0 the seed will be generated. Seed value will be logged during the solve.
	*/
	UFUNCTION(BlueprintCallable, Category = "WFCFunctions")
	AActor* Collapse(int32 TryCount = 1, int32 RandomSeed = 0);

	/**
	* Initialize WFC process which sets up Tiles and RemainingTiles arrays
	* Pre-populates Tiles with StarterOptions, BorderOptions and InitialTiles
	* @param Tiles Array of tiles (by ref)
	* @param RemainingTiles Array of remaining tile indices.  Semi-sorted: Min Entropy tiles at the front, the rest remains unsorted (by ref)
	*/
	UFUNCTION(BlueprintCallable, Category = "WFCFunctions")
	void InitializeWFC(TArray<FWaveFunctionCollapseTile>& Tiles, TArray<int32>& RemainingTiles);
	
	/**
	* Observation phase: 
	* This process randomly selects one tile from minimum entropy tiles
	* then randomly selects a valid option for that tile
	* @param Tiles Array of tiles (by ref)
	* @param RemainingTiles Array of remaining tile indices.  Semi-sorted: Min Entropy tiles at the front, the rest remains unsorted (by ref)
	* @param ObservationQueue Array to store tiles that need to be checked whether remaining options are affected during propagation phase (by ref)
	*/
	UFUNCTION(BlueprintCallable, Category = "WFCFunctions")
	bool Observe(TArray<FWaveFunctionCollapseTile>& Tiles, 
		TArray<int32>& RemainingTiles, 
		TMap<int32, FWaveFunctionCollapseQueueElement>& ObservationQueue,
		int32 RandomSeed);
	
	/**
	* Propagation phase: 
	* This process checks if the selection made during the observation is valid by checking constraint validity with neighboring tiles. 
	* Neighboring tiles may reduce their remaining options to include only valid options.
	* If the remaining options of a tile were modified, the neighboring tiles of the modified tile will be added to a queue.
	* During this process, if any contradiction (a tile with zero remaining options) is encountered, the current solve will fail.
	* @param Tiles Array of tiles (by ref)
	* @param RemainingTiles Array of remaining tile indices.  Semi-sorted: Min Entropy tiles at the front, the rest remains unsorted (by ref)
	* @param ObservationQueue Array to store tiles that need to be checked whether remaining options are affected (by ref)
	* @param PropagationCount Counter for propagation passes
	*/
	UFUNCTION(BlueprintCallable, Category = "WFCFunctions")
	bool Propagate(TArray<FWaveFunctionCollapseTile>& Tiles, 
		TArray<int32>& RemainingTiles, 
		TMap<int32, FWaveFunctionCollapseQueueElement>& ObservationQueue, 
		int32& PropagationCount);
	
	/**
	* Recursive Observation and Propagation cycle
	* @param Tiles Array of tiles (by ref)
	* @param RemainingTiles Array of remaining tile indices (by ref)
	* @param ObservationQueue Array to store tiles that need to be checked whether remaining options are affected (by ref)
	*/
	UFUNCTION(BlueprintCallable, Category = "WFCFunctions")
	bool ObservationPropagation(TArray<FWaveFunctionCollapseTile>& Tiles, 
		TArray<int32>& RemainingTiles,
		TMap<int32, FWaveFunctionCollapseQueueElement>& ObservationQueue,
		int32 RandomSeed);

	/**
	* Derive grid from the bounds of an array of transforms
	* Assumptions:
	*	-Transforms can only represent a single grid
	*   -Sets empty starter option if there is a valid grid position with no transform
	*   -Orientation is determined by the yaw of the first transform in the array
	* @param Transforms Array of transforms (by ref)
	*/
	UFUNCTION(BlueprintCallable, Category = "WFCFunctions")
	void DeriveGridFromTransformBounds(const TArray<FTransform>& Transforms);

	/**
	* Derive grid from an array of transforms
	* Assumptions:
	*   -Every transform represents the center point of a tile position
	*   -Sets empty starter option if there is a valid grid position with no transform
	*   -Orientation is determined by the yaw of the first transform in the array
	* @param Transforms Array of transforms (by ref)
	*/
	UFUNCTION(BlueprintCallable, Category = "WFCFunctions")
	void DeriveGridFromTransforms(const TArray<FTransform>& Transforms);

private:

	/**
	* Builds the Initial Tile which is a tile containing all possible options
	* @param InitialTile The Initial Tile (by ref)
	*/
	bool BuildInitialTile(FWaveFunctionCollapseTile& InitialTile);

	/**
	* Get valid options for a border tile
	* @param Position Position of border tile
	* @param TmpInitialOptions Initial options which should contain all possible options
	*/
	TArray<FWaveFunctionCollapseOption> GetInnerBorderOptions(FIntVector Position, const TArray<FWaveFunctionCollapseOption>& InitialOptions);

	/**
	* Used in GetBorderOptions to gather invalid options for border tiles that should be removed from the InitialOptions set
	* @param Adjacency Direction from Exterior Border to Inner Border
	* @param InInitialOptions This should be the InitialOptions
	* @param OutBorderOptionsToRemove Array containing gathered options to remove
	*/
	void GatherInnerBorderOptionsToRemove(EWaveFunctionCollapseAdjacency Adjacency, const TArray<FWaveFunctionCollapseOption>& InitialOptions, TArray<FWaveFunctionCollapseOption>& OutBorderOptionsToRemove);
	
	/**
	* Checks if a position is an inner border
	* @param Position
	*/
	bool IsPositionInnerBorder(FIntVector Position);
	
	/**
	* Used in Observe and Propagate to add adjacent indices to a queue
	* @param CenterIndex Index of the center object
	* @param RemainingTiles Used to check if index still remains in RemainingTiles
	* @param OutQueue Queue to add indices to
	*/
	void AddAdjacentIndicesToQueue(int32 CenterIndex, const TArray<int32>& RemainingTiles, TMap<int32,FWaveFunctionCollapseQueueElement>& OutQueue);
	
	/**
	* Add an Instance Component with a given name
	* @param Actor Actor to add component to
	* @param ComponentClass
	* @param ComponentName
	*/
	UActorComponent* AddNamedInstanceComponent(AActor* Actor, TSubclassOf<UActorComponent> ComponentClass, FName ComponentName);
	
	/**
	* Spawn an actor given an array of successfully solved tiles
	* @param Tiles Successfully solved array of tiles
	*/
	AActor* SpawnActorFromTiles(const TArray<FWaveFunctionCollapseTile>& Tiles);
	
	/**
	* Returns true if no remaining options in given tiles are an empty/void option or included in the SpawnExclusion list
	* @param Tiles Successfully solved array of tiles
	*/
	bool AreAllTilesNonSpawnable(const TArray<FWaveFunctionCollapseTile>& Tiles);
};