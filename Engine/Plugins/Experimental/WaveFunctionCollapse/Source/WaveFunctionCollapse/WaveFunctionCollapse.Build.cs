// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WaveFunctionCollapse : ModuleRules
{
	public WaveFunctionCollapse(ReadOnlyTargetRules Target) : base(Target)
	{
		ShortName = "WFC";

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"EditorSubsystem",
				"EditorStyle",
				"LevelEditor",
				"EditorFramework",
				"UnrealEd",
				"Blutility",
				"UMG",
				"UMGEditor",
				"AssetTools",
				"PropertyEditor",
			}
		);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"InputCore",
				"Engine",
				"Slate",
				"SlateCore",
			}
		);
	}
}
