{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Wave Function Collapse (Experimental)",
	"Description": "Wave Function Collapse tools for tile-based model synthesis",
	"Category": "Experimental",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://www.epicgames.com/",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "WaveFunctionCollapse",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	]
}